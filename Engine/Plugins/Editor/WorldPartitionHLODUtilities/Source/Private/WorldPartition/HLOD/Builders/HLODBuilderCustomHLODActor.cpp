// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/Builders/HLODBuilderCustomHLODActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODBuilderCustomHLODActor)

UHLODBuilderCustomHLODActorSettings::UHLODBuilderCustomHLODActorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UHLODBuilderCustomHLODActor::UHLODBuilderCustomHLODActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSubclassOf<UHLODBuilderSettings> UHLODBuilderCustomHLODActor::GetSettingsClass() const
{
	return UHLODBuilderCustomHLODActorSettings::StaticClass();
}

TArray<UActorComponent*> UHLODBuilderCustomHLODActor::Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const
{
	return {};
}