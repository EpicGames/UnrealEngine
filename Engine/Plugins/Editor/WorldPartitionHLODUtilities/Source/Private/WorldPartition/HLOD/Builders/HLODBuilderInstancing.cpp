// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/Builders/HLODBuilderInstancing.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"

#include "Serialization/ArchiveCrc32.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODBuilderInstancing)


UHLODBuilderInstancingSettings::UHLODBuilderInstancingSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bDisallowNanite(false)
	, InstanceFilteringType(EInstanceFilteringType::FilterNone)
	, MinimumExtent(0)
	, MinimumArea(0)
	, MinimumVolume(0)
{
}

void UHLODBuilderInstancingSettings::ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const
{
	// Base key, changing this will force a rebuild of all HLODs from this builder
	FString HLODBaseKey = TEXT("5197F75BF850402EABA5D29E4D8BB1DA");
	InHashBuilder.HashField(HLODBaseKey, TEXT("MeshInstancingBaseKey"));
	InHashBuilder.HashField(bDisallowNanite, TEXT("DissallowNanite"));

	if (InstanceFilteringType != EInstanceFilteringType::FilterNone)
	{
		InHashBuilder.HashField(InstanceFilteringType, TEXT("InstanceFilteringType"));
		switch (InstanceFilteringType)
		{
		case EInstanceFilteringType::FilterMinimumExtent:
			InHashBuilder.HashField(MinimumExtent, TEXT("MinimumExtent"));
			break;
		case EInstanceFilteringType::FilterMinimumArea:
			InHashBuilder.HashField(MinimumArea, TEXT("MinimumArea"));
			break;
		case EInstanceFilteringType::FilterMinimumVolume:
			InHashBuilder.HashField(MinimumVolume, TEXT("MinimumVolume"));
			break;
		default:
			unimplemented();
		}
	}
}


UHLODBuilderInstancing::UHLODBuilderInstancing(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSubclassOf<UHLODBuilderSettings> UHLODBuilderInstancing::GetSettingsClass() const
{
	return UHLODBuilderInstancingSettings::StaticClass();
}

TArray<UActorComponent*> UHLODBuilderInstancing::Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const
{
	const UHLODBuilderInstancingSettings& InstancingSettings = *CastChecked<UHLODBuilderInstancingSettings>(HLODBuilderSettings);

	int32 NumInstancesTotal = 0;
	int32 NumInstancesRejected = 0;

	auto FilterInstances = [&InstancingSettings, &NumInstancesTotal, &NumInstancesRejected](const FBox& InInstanceBounds)
	{
		bool bPassFilter = true;

		switch (InstancingSettings.InstanceFilteringType)
		{
			case EInstanceFilteringType::FilterMinimumExtent:
			{
				double MaxExtent = InInstanceBounds.GetExtent().GetMax();
				bPassFilter = MaxExtent >= InstancingSettings.MinimumExtent;
				break;
			}
			case EInstanceFilteringType::FilterMinimumArea:
			{
				FVector Extent = InInstanceBounds.GetExtent();
				double Area = 8 * (Extent.X * Extent.Y + Extent.X * Extent.Z + Extent.Y * Extent.Z);
				bPassFilter = Area >= InstancingSettings.MinimumArea;
				break;
			}
			case EInstanceFilteringType::FilterMinimumVolume:
			{
				double Volume = InInstanceBounds.GetVolume();
				bPassFilter = Volume >= InstancingSettings.MinimumVolume;
				break;
			}
		}

		NumInstancesTotal++;
		NumInstancesRejected += bPassFilter ? 0 : 1;

		return bPassFilter;
	};

	TArray<UActorComponent*> HLODComponents = UHLODBuilder::BatchInstances(InSourceComponents, FilterInstances);

	if (NumInstancesRejected > 0)
	{
		UE_LOGF(LogHLODBuilder, Log, "UHLODBuilderInstancing: Filter rejected %d out of %d instances", NumInstancesRejected, NumInstancesTotal);
	}

	// If requested, disallow Nanite on components whose mesh is Nanite enabled
	if (InstancingSettings.bDisallowNanite)
	{
		for (UActorComponent* HLODComponent : HLODComponents)
		{
			if (UStaticMeshComponent* SMComponent = Cast<UStaticMeshComponent>(HLODComponent))
			{
				if (UStaticMesh* StaticMesh = SMComponent->GetStaticMesh())
				{
					if (!SMComponent->bDisallowNanite && StaticMesh->HasValidNaniteData())
					{
						SMComponent->bDisallowNanite = true;
						SMComponent->MarkRenderStateDirty();
					}
				}
			}
		}
	}

	return HLODComponents;
}
