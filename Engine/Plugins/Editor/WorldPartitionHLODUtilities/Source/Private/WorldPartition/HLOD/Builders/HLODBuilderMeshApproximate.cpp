// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/Builders/HLODBuilderMeshApproximate.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"

#include "Algo/ForEach.h"
#include "Components/StaticMeshComponent.h"
#include "UObject/Package.h"

#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "MaterialUtilities.h"

#include "Modules/ModuleManager.h"
#include "IGeometryProcessingInterfacesModule.h"
#include "GeometryProcessingInterfaces/ApproximateActors.h"

#include "Engine/HLODProxy.h"
#include "Serialization/ArchiveCrc32.h"
#include "ObjectTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODBuilderMeshApproximate)


UHLODBuilderMeshApproximate::UHLODBuilderMeshApproximate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UHLODBuilderMeshApproximateSettings::UHLODBuilderMeshApproximateSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsTemplate())
	{
		HLODMaterial = GEngine->DefaultHLODFlattenMaterial;
	}
#endif
}

void UHLODBuilderMeshApproximateSettings::ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const
{
	// Base key, changing this will force a rebuild of all HLODs from this builder
	FString HLODBaseKey = TEXT("4E9BB187B38645A08B424895C5268B8A");
	InHashBuilder.HashField(HLODBaseKey, TEXT("MeshApproximateBaseKey"));
	InHashBuilder.HashField(MeshApproximationSettings, TEXT("MeshApproximationSettingsHash"));
	InHashBuilder << HLODMaterial;
}

TSubclassOf<UHLODBuilderSettings> UHLODBuilderMeshApproximate::GetSettingsClass() const
{
	return UHLODBuilderMeshApproximateSettings::StaticClass();
}

TArray<UActorComponent*> UHLODBuilderMeshApproximate::Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHLODBuilderMeshApproximate::Build);

	IGeometryProcessing_ApproximateActors::FInput Input;
	Input.Components = InSourceComponents;

	IGeometryProcessingInterfacesModule& GeomProcInterfaces = FModuleManager::Get().LoadModuleChecked<IGeometryProcessingInterfacesModule>("GeometryProcessingInterfaces");
	IGeometryProcessing_ApproximateActors* ApproxActorsAPI = GeomProcInterfaces.GetApproximateActorsImplementation();

	//
	// Construct options for ApproximateActors operation
	//

	const UHLODBuilderMeshApproximateSettings* MeshApproximateSettings = CastChecked<UHLODBuilderMeshApproximateSettings>(HLODBuilderSettings);
	FMeshApproximationSettings UseSettings = MeshApproximateSettings->MeshApproximationSettings; // Make a copy as we may tweak some values
	UMaterialInterface* HLODMaterial = MeshApproximateSettings->HLODMaterial;

	// When using automatic texture sizing based on draw distance, use the MinVisibleDistance for this HLOD.
	if (UseSettings.MaterialSettings.TextureSizingType == TextureSizingType_AutomaticFromMeshDrawDistance)
	{
		UseSettings.MaterialSettings.MeshMinDrawDistance = InHLODBuildContext.MinVisibleDistance;
	}

	IGeometryProcessing_ApproximateActors::FOptions Options = ApproxActorsAPI->ConstructOptions(UseSettings);
	Options.BasePackagePath = InHLODBuildContext.AssetsOuter->GetPackage()->GetName();
	Options.bGenerateLightmapUVs = false;
	Options.bCreatePhysicsBody = false;
	Options.bBuildReversedIndexBuffer = false;

	// Material baking settings
	Options.BakeMaterial = HLODMaterial;
	if (!FMaterialUtilities::IsValidFlattenMaterial(Options.BakeMaterial))
	{
		Options.BakeMaterial = GEngine->DefaultFlattenMaterial;
	}
	Options.BaseColorTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Diffuse, Options.BakeMaterial));
	Options.NormalTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Normal, Options.BakeMaterial));
	Options.MetallicTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Metallic, Options.BakeMaterial));
	Options.RoughnessTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Roughness, Options.BakeMaterial));
	Options.SpecularTexParamName = FName(FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties::Specular, Options.BakeMaterial));
	Options.EmissiveTexParamName = FName("EmissiveHDRTexture"); // TODO - Approximate actors should look up if the material sampler is expecting an HDR texture and capture accordingly
	Options.bUsePackedMRS = true;
	Options.PackedMRSTexParamName = FName("PackedTexture");

	// Compute texel density if needed, depending on the TextureSizingType setting
	TArray<UPrimitiveComponent*> PrimitiveComponents = FilterComponents<UPrimitiveComponent>(InSourceComponents);
	if (UseSettings.MaterialSettings.ResolveTexelDensity(PrimitiveComponents, Options.MeshTexelDensity))
	{ 
		Options.TextureSizePolicy = IGeometryProcessing_ApproximateActors::ETextureSizePolicy::TexelDensity;
	}

	// Use temp packages - Needed to allow proper replacement of existing assets (performed below)
	const FString NewAssetNamePrefix(TEXT("NEWASSET_"));
	FString PackagePath = InHLODBuildContext.AssetsOuter->GetPackage()->GetName();
	FString AssetName = InHLODBuildContext.AssetsBaseName;
	Options.BasePackagePath = PackagePath / NewAssetNamePrefix + AssetName;

	// run actor approximation computation
	IGeometryProcessing_ApproximateActors::FResults Results;
	ApproxActorsAPI->ApproximateActors(Input, Options, Results);

	TArray<UActorComponent*> Components;
	if (Results.ResultCode == IGeometryProcessing_ApproximateActors::EResultCode::Success)
	{
		auto ProcessNewAsset = [&InHLODBuildContext, &NewAssetNamePrefix](UObject* NewAsset)
		{
			// Move asset out of the temp package and into its final package
			{
				FString AssetName = NewAsset->GetName();
				AssetName.RemoveFromStart(NewAssetNamePrefix);

				UObject* AssetToReplace = StaticFindObjectFast(UObject::StaticClass(), InHLODBuildContext.AssetsOuter, *AssetName);
				if (AssetToReplace)
				{
					// Move the previous asset to the transient package
					AssetToReplace->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
				}

				UPackage* TempPackage = NewAsset->GetPackage();

				// Rename the asset to its final destination
				NewAsset->Rename(*AssetName, InHLODBuildContext.AssetsOuter, REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
				NewAsset->ClearFlags(RF_Public | RF_Standalone);
				NewAsset->SetFlags(InHLODBuildContext.AssetsObjectFlags);

				// Clean up flags on the temp package. It is not useful anymore.
				TempPackage->ClearDirtyFlag();
				TempPackage->SetFlags(RF_Transient);
				TempPackage->ClearFlags(RF_Public | RF_Standalone);
			}
		};
	
		Algo::ForEach(Results.NewMeshAssets, ProcessNewAsset);
		Algo::ForEach(Results.NewMaterials, ProcessNewAsset);
		Algo::ForEach(Results.NewTextures, ProcessNewAsset);

		for (UStaticMesh* StaticMesh : Results.NewMeshAssets)
		{
			UStaticMeshComponent* Component = NewObject<UStaticMeshComponent>();
			Component->SetStaticMesh(StaticMesh);

			Components.Add(Component);
		}

		for (UMaterialInterface* Material : Results.NewMaterials)
		{
			UMaterialInstance* MaterialInst = CastChecked<UMaterialInstance>(Material);

			FStaticParameterSet StaticParameterSet;
			
			auto SetStaticSwitch = [&StaticParameterSet](FName ParamName, bool bSet)
			{
				if (bSet)
				{
					FStaticSwitchParameter SwitchParameter;
					SwitchParameter.ParameterInfo.Name = ParamName;
					SwitchParameter.Value = true;
					SwitchParameter.bOverride = true;
					StaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);
				}
			};

			// Set proper switches needed by our base flatten material
			SetStaticSwitch("UseBaseColor", Options.bBakeBaseColor);
			SetStaticSwitch("UseDiffuse", Options.bBakeBaseColor);
			SetStaticSwitch("UseRoughness", Options.bBakeRoughness);
			SetStaticSwitch("UseMetallic", Options.bBakeMetallic);
			SetStaticSwitch("UseSpecular", Options.bBakeSpecular);
			SetStaticSwitch("UseEmissive", Options.bBakeEmissive);
			SetStaticSwitch("UseEmissiveColor", Options.bBakeEmissive);
			SetStaticSwitch("UseEmissiveHDR", Options.bBakeEmissive);
			SetStaticSwitch("UseNormal", Options.bBakeNormalMap);
			SetStaticSwitch("PackMetallic", Options.bUsePackedMRS);
			SetStaticSwitch("PackSpecular", Options.bUsePackedMRS);
			SetStaticSwitch("PackRoughness", Options.bUsePackedMRS);

			// Force initializing the static permutations according to the switches we have set
			MaterialInst->UpdateStaticPermutation(StaticParameterSet);
			MaterialInst->InitStaticPermutation();
			MaterialInst->PostEditChange();
		}
	}

	return Components;
}

