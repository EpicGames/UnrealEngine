// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/Builders/HLODBuilderMeshMerge.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"

#include "Algo/ForEach.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "Components/StaticMeshComponent.h"

#include "IMaterialBakingModule.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"
#include "Modules/ModuleManager.h"

#include "Materials/Material.h"
#include "Engine/HLODProxy.h"
#include "Serialization/ArchiveCrc32.h"

#include "WorldPartition/HLOD/HLODEditorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODBuilderMeshMerge)


UHLODBuilderMeshMerge::UHLODBuilderMeshMerge(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UHLODBuilderMeshMergeSettings::UHLODBuilderMeshMergeSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsTemplate())
	{
		HLODMaterial = GEngine->DefaultHLODFlattenMaterial;
	}
#endif

	if (IsTemplate())
	{
		HLOD_ADD_STRUCT_SETTING_FILTER(BasicSettings, FMaterialProxySettings, bNormalMap);
		HLOD_ADD_STRUCT_SETTING_FILTER(BasicSettings, FMaterialProxySettings, bTangentMap);
		HLOD_ADD_STRUCT_SETTING_FILTER(BasicSettings, FMaterialProxySettings, bMetallicMap);
		HLOD_ADD_STRUCT_SETTING_FILTER(BasicSettings, FMaterialProxySettings, bRoughnessMap);
		HLOD_ADD_STRUCT_SETTING_FILTER(BasicSettings, FMaterialProxySettings, bSpecularMap);
		HLOD_ADD_STRUCT_SETTING_FILTER(BasicSettings, FMaterialProxySettings, bEmissiveMap);

		HLOD_ADD_STRUCT_SETTING_FILTER(BasicSettings, FMeshMergingSettings, MaterialSettings);

		HLOD_ADD_CLASS_SETTING_FILTER(BasicSettings, UHLODBuilderMeshMergeSettings, MeshMergeSettings);
	}
}

void UHLODBuilderMeshMergeSettings::ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const
{
	// Base mesh merge key, changing this will force a rebuild of all HLODs from this builder
	FString HLODBaseKey = TEXT("F906DCC49BCB4102A821B20D64C6E93B");
	InHashBuilder.HashField(HLODBaseKey, TEXT("MeshMergeBaseKey"));

	InHashBuilder.HashField(MeshMergeSettings, TEXT("MeshMergeSettingsHash"));

	if (MeshMergeSettings.bMergeMaterials)
	{
		IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");
		InHashBuilder.HashField(Module.GetCRC(), TEXT("MaterialBakingModuleHash"));

		static const auto MeshMergeUtilitiesUVGenerationMethodCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("MeshMergeUtilities.UVGenerationMethod"));
		int32 MeshMergeUtilitiesUVGenerationMethod = (MeshMergeUtilitiesUVGenerationMethodCVar != nullptr) ? MeshMergeUtilitiesUVGenerationMethodCVar->GetInt() : 0;
		InHashBuilder.HashField(MeshMergeUtilitiesUVGenerationMethod, TEXT("MeshMergeUtilitiesUVGenerationMethod"));
	}

	InHashBuilder << HLODMaterial;
}

TSubclassOf<UHLODBuilderSettings> UHLODBuilderMeshMerge::GetSettingsClass() const
{
	return UHLODBuilderMeshMergeSettings::StaticClass();
}

TArray<UActorComponent*> UHLODBuilderMeshMerge::Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHLODBuilderMeshMerge::CreateComponents);

	TArray<UPrimitiveComponent*> SourcePrimitiveComponents = FilterComponents<UPrimitiveComponent>(InSourceComponents);

	TArray<UObject*> Assets;
	FVector MergedActorLocation;

	const UHLODBuilderMeshMergeSettings* MeshMergeSettings = CastChecked<UHLODBuilderMeshMergeSettings>(HLODBuilderSettings);
	FMeshMergingSettings UseSettings = MeshMergeSettings->MeshMergeSettings; // Make a copy as we may tweak some values
	UMaterialInterface* HLODMaterial = MeshMergeSettings->HLODMaterial;

	// When using automatic texture sizing based on draw distance, use the MinVisibleDistance for this HLOD.
	if (UseSettings.MaterialSettings.TextureSizingType == TextureSizingType_AutomaticFromMeshDrawDistance)
	{
		UseSettings.MaterialSettings.MeshMinDrawDistance = InHLODBuildContext.MinVisibleDistance;
	}

	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	MeshMergeUtilities.MergeComponentsToStaticMesh(SourcePrimitiveComponents, InHLODBuildContext.BuildWorld, UseSettings, HLODMaterial, InHLODBuildContext.AssetsOuter->GetPackage(), InHLODBuildContext.AssetsBaseName, Assets, MergedActorLocation, 0.25f, false);

	UStaticMeshComponent* Component = nullptr;
	Algo::ForEach(Assets, [this, &Component, &MergedActorLocation, &InHLODBuildContext](UObject* Asset)
	{
		Asset->ClearFlags(RF_Public | RF_Standalone);

		if (Cast<UStaticMesh>(Asset))
		{
			Asset->SetFlags(InHLODBuildContext.AssetsObjectFlags);
			Component = NewObject<UStaticMeshComponent>();
			Component->SetStaticMesh(static_cast<UStaticMesh*>(Asset));
			Component->SetWorldLocation(MergedActorLocation);
		}
	});

	return { Component };
}

