// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/Builders/HLODBuilderMeshSimplify.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"

#include "Algo/ForEach.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "Components/StaticMeshComponent.h"

#include "IMaterialBakingModule.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"
#include "Modules/ModuleManager.h"

#include "Materials/Material.h"
#include "Engine/HLODProxy.h"
#include "PrimitiveDrawingUtils.h"
#include "Serialization/ArchiveCrc32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODBuilderMeshSimplify)


UHLODBuilderMeshSimplify::UHLODBuilderMeshSimplify(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UHLODBuilderMeshSimplifySettings::UHLODBuilderMeshSimplifySettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	if (!IsTemplate())
	{
		HLODMaterial = GEngine->DefaultHLODFlattenMaterial;
	}
#endif
}

void UHLODBuilderMeshSimplifySettings::ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const
{
	// Base mesh simplify key, changing this will force a rebuild of all HLODs from this builder
	FString HLODBaseKey = TEXT("833ECCC91448424CACC620661712928A");
	InHashBuilder.HashField(HLODBaseKey, TEXT("MeshSimplifyBaseKey"));

	InHashBuilder.HashField(MeshSimplifySettings, TEXT("MeshSimplifySettingsHash"));

	IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");
	uint32 MaterialBakingModuleCRC = Module.GetCRC();
	InHashBuilder.HashField(MaterialBakingModuleCRC, TEXT("MaterialBakingModuleCRC"));

	static const auto MeshMergeUtilitiesUVGenerationMethodCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("MeshMergeUtilities.UVGenerationMethod"));
	int32 MeshMergeUtilitiesUVGenerationMethod = (MeshMergeUtilitiesUVGenerationMethodCVar != nullptr) ? MeshMergeUtilitiesUVGenerationMethodCVar->GetInt() : 0;
	InHashBuilder.HashField(MeshMergeUtilitiesUVGenerationMethod, TEXT("MeshMergeUtilitiesUVGenerationMethod"));

	InHashBuilder << HLODMaterial;
}

TSubclassOf<UHLODBuilderSettings> UHLODBuilderMeshSimplify::GetSettingsClass() const
{
	return UHLODBuilderMeshSimplifySettings::StaticClass();
}

TArray<UActorComponent*> UHLODBuilderMeshSimplify::Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHLODBuilderMeshSimplifySettings::Build);

	TArray<UStaticMeshComponent*> StaticMeshComponents = FilterComponents<UStaticMeshComponent>(InSourceComponents);

	const UHLODBuilderMeshSimplifySettings* MeshSimplifySettings = CastChecked<UHLODBuilderMeshSimplifySettings>(HLODBuilderSettings);
	FMeshProxySettings UseSettings = MeshSimplifySettings->MeshSimplifySettings; // Make a copy as we may tweak some values
	UMaterialInterface* HLODMaterial = MeshSimplifySettings->HLODMaterial;

	// When using automatic texture sizing based on draw distance, use the MinVisibleDistance for this HLOD.
	if (UseSettings.MaterialSettings.TextureSizingType == TextureSizingType_AutomaticFromMeshDrawDistance)
	{
		UseSettings.MaterialSettings.MeshMinDrawDistance = InHLODBuildContext.MinVisibleDistance;
	}

	// Generate a projection matrix.
	static const float ScreenX = 1920;
	static const float ScreenY = 1080;
	static const float HalfFOVRad = FMath::DegreesToRadians(45.0f);
	static const FMatrix ProjectionMatrix = FPerspectiveMatrix(HalfFOVRad, ScreenX, ScreenY, 0.01f);

	// Gather bounds of the input components
	auto GetComponentsBounds = [&]() -> FBoxSphereBounds
	{
		FBoxSphereBounds::Builder BoundsBuilder;

		for (UActorComponent* Component : InSourceComponents)
		{
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
			{
				BoundsBuilder += SceneComponent->Bounds;
			}
		}

		return BoundsBuilder;
	};

	float ScreenSizePercent = ComputeBoundsScreenSize(FVector::ZeroVector, static_cast<float>(GetComponentsBounds().SphereRadius), FVector(0.0f, 0.0f, InHLODBuildContext.MinVisibleDistance), ProjectionMatrix);
	UseSettings.ScreenSize = FMath::RoundToInt(ScreenSizePercent * ScreenX);

	TArray<UObject*> Assets;
	FCreateProxyDelegate ProxyDelegate;
	ProxyDelegate.BindLambda([&Assets](const FGuid Guid, TArray<UObject*>& InAssetsCreated) { Assets = InAssetsCreated; });

	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	MeshMergeUtilities.CreateProxyMesh(StaticMeshComponents, UseSettings, HLODMaterial, InHLODBuildContext.AssetsOuter->GetPackage(), InHLODBuildContext.AssetsBaseName, FGuid::NewGuid(), ProxyDelegate, true);


	TArray<UActorComponent*> Components;

	Algo::ForEach(Assets, [this, &Components, &InHLODBuildContext](UObject* Asset)
	{
		Asset->ClearFlags(RF_Public | RF_Standalone);

		if (Cast<UStaticMesh>(Asset))
		{
			Asset->SetFlags(InHLODBuildContext.AssetsObjectFlags);
			UStaticMeshComponent* SMComponent = NewObject<UStaticMeshComponent>();
			SMComponent->SetStaticMesh(static_cast<UStaticMesh*>(Asset));

			Components.Add(SMComponent);
		}
	});
	
	return Components;
}

