// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/Modifiers/HLODModifierMeshDestruction.h"

#include "Components/StaticMeshComponent.h"
#include "IMeshMergeExtension.h"
#include "IMeshMergeUtilities.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MeshMergeDataTracker.h"
#include "MeshMergeModule.h"
#include "Modules/ModuleManager.h"
#include "StaticMeshAttributes.h"
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "WorldPartition/HLOD/HLODDestruction.h"
#include "WorldPartition/HLOD/HLODInstancedStaticMeshComponent.h"
#include "WorldPartition/HLOD/DestructibleHLODComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODModifierMeshDestruction)


UWorldPartitionHLODModifierMeshDestruction::UWorldPartitionHLODModifierMeshDestruction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}



bool UWorldPartitionHLODModifierMeshDestruction::CanModifyHLOD(TSubclassOf<UHLODBuilder> InHLODBuilderClass) const
{
	return true;
}


// Use vertex color attributes to store component indices
// In the material, this allows us to mask destructed building parts
class FDestructionMeshMergeExtension : public IMeshMergeExtension
{
public:
	TArray<FName> DestructibleActors;
	UMaterialInterface* DestructibleMaterial = nullptr;

	FDestructionMeshMergeExtension()
	{
		FModuleManager::LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities().RegisterExtension(this);
	}

	virtual ~FDestructionMeshMergeExtension()
	{
		FModuleManager::LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities().UnregisterExtension(this);
	}

	virtual void OnCreatedMergedRawMeshes(const TArray<UStaticMeshComponent*>& MergedComponents, const class FMeshMergeDataTracker& DataTracker, TArray<FMeshDescription>& MergedMeshLODs) override
	{
		check(DestructibleActors.IsEmpty());

		// Clear vertex colors
		FMeshDescription& MergedMesh = MergedMeshLODs[0];
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = MergedMesh.VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color);
		for (FVertexInstanceID VertexInstanceID : MergedMesh.VertexInstances().GetElementIDs())
		{
			VertexInstanceColors[VertexInstanceID] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		}

		const uint32 MergedMeshVertexInstancesNum = MergedMesh.VertexInstances().Num();

		for (int32 ComponentIndex = 0; ComponentIndex < MergedComponents.Num(); ++ComponentIndex)
		{
			// Start index
			const uint32 ComponentVertexColorIndexStart = DataTracker.GetComponentToWedgeMappng(ComponentIndex, /*LODIndex=*/0);
			if (ComponentVertexColorIndexStart == INDEX_NONE)
			{
				continue;
			}

			// End index
			uint32 ComponentVertexColorIndexEnd = MergedMeshVertexInstancesNum;
			for (int32 NextComponentIndex = ComponentIndex + 1; NextComponentIndex < MergedComponents.Num(); ++NextComponentIndex)
			{
				// Skip invalid entries
				uint32 NextComponentVertexColorIndexStart = DataTracker.GetComponentToWedgeMappng(NextComponentIndex, /*LODIndex=*/0);
				if (NextComponentVertexColorIndexStart != INDEX_NONE)
				{
					ComponentVertexColorIndexEnd = NextComponentVertexColorIndexStart;
					break;
				}
			}

			FColor ActorColor = FColor::Black;
			UStaticMeshComponent* StaticMeshComponent = MergedComponents[ComponentIndex];
			if (AActor* Actor = StaticMeshComponent->GetOwner())
			{
				if (Actor->Implements<UWorldPartitionDestructibleInHLODInterface>())
				{
					uint32 ActorIndex = DestructibleActors.AddUnique(Actor->GetFName());
					ActorColor.DWColor() = ActorIndex + 1;
				}
			}

			for (uint32 ComponentVertexColorIndex = ComponentVertexColorIndexStart; ComponentVertexColorIndex < ComponentVertexColorIndexEnd; ++ComponentVertexColorIndex)
			{
				VertexInstanceColors[FVertexInstanceID(ComponentVertexColorIndex)] = FLinearColor(ActorColor);
			}
		}
	}

	virtual void OnCreatedProxyMaterial(const TArray<UStaticMeshComponent*>& MergedComponents, UMaterialInterface* ProxyMaterial) override
	{
		UMaterialInstanceConstant* Instance = Cast<UMaterialInstanceConstant>(ProxyMaterial);
		if (!Instance)
		{
			return;
		}

		TSet<AActor*> DestructibleActorsSet;
		for (UStaticMeshComponent* Component : MergedComponents)
		{
			if (Component->GetOwner()->Implements<UWorldPartitionDestructibleInHLODInterface>())
			{
				DestructibleActorsSet.Add(Component->GetOwner());
			}
		}

		// Ensure a destructible material is used
		const static FName EnableInstanceDestroyingParamName(TEXT("EnableInstanceDestroying"));
		FGuid ParamGUID;
		bool bEnableInstanceDestroying = false;
		bool bFoundParam = Instance->GetStaticSwitchParameterValue(EnableInstanceDestroyingParamName, bEnableInstanceDestroying, ParamGUID);
		if (bFoundParam && bEnableInstanceDestroying)
		{
			Instance->SetScalarParameterValueEditorOnly(FMaterialParameterInfo("NumInstances"), static_cast<float>(FMath::RoundUpToPowerOfTwo(DestructibleActorsSet.Num())));
			Instance->BasePropertyOverrides.TwoSided = false;
			Instance->BasePropertyOverrides.bOverride_TwoSided = true;

			FStaticParameterSet CurrentParameters = Instance->GetStaticParameters();

			Instance->UpdateStaticPermutation(CurrentParameters);
			Instance->InitStaticPermutation();

			DestructibleMaterial = Instance;
		}
	}
};

void UWorldPartitionHLODModifierMeshDestruction::BeginHLODBuild(const FHLODBuildContext& InHLODBuildContext)
{
	CachedWorldPosition = InHLODBuildContext.WorldPosition;

	DestructionMeshMergeExtension = MakeUnique<FDestructionMeshMergeExtension>();
}

void UWorldPartitionHLODModifierMeshDestruction::EndHLODBuild(TArray<UActorComponent*>& InOutComponents)
{
	UMaterialInterface* DestructibleMaterial = DestructionMeshMergeExtension->DestructibleMaterial;
	TArray<FName> DestructibleActors = DestructionMeshMergeExtension->DestructibleActors;
	FHLODInstancingPackedMappingData PackedISMData;

	TArray<UHLODInstancedStaticMeshComponent*> HLODISMCs;
	Algo::TransformIf(InOutComponents, HLODISMCs, [](UActorComponent* InComponent) { return InComponent->IsA<UHLODInstancedStaticMeshComponent>(); }, [](UActorComponent* InComponent) { return CastChecked<UHLODInstancedStaticMeshComponent>(InComponent); });
	
	if (!HLODISMCs.IsEmpty())
	{
		struct FISMActorMapping
		{
			uint32	ActorIndex;			// Index of the source actor in the DestructibleActors array
			uint32	ComponentIndex;		// Index of the HLOD component in the HLODISMCs array
			uint32	InstancingStart;	// Start of the range of instances associated with ActorIndex in the HLOD component
			uint32	InstancingCount;	// Number of instances associated with ActorIndex in the HLOD component
		};
		TArray<FISMActorMapping> ISMActorMapping;

		TArray<UHLODInstancedStaticMeshComponent*> DestructibleHLODISMCs;

		// Build the full ISM to actor mapping by retrieving the mapping data from the UHLODInstancedStaticMeshComponent components
		for (UHLODInstancedStaticMeshComponent* Component : HLODISMCs)
		{
			for (const FISMComponentBatcher::FComponentToInstancesMapping& ComponentToInstancesMapping : Component->GetSourceComponentsToInstancesMap())
			{
				const UActorComponent* SourceComponent = ComponentToInstancesMapping.Component;
				const AActor* SourceActor = SourceComponent->GetOwner();
				if (SourceActor->Implements<UWorldPartitionDestructibleInHLODInterface>())
				{
					// Add component to the destructible HLOD ISMC array if it wasn't already added
					if (DestructibleHLODISMCs.IsEmpty() || DestructibleHLODISMCs.Last() != Component)
					{
						DestructibleHLODISMCs.Add(Component);
					}

					uint32 ActorIndex = DestructibleActors.AddUnique(SourceActor->GetFName());
					ISMActorMapping.Emplace(ActorIndex, DestructibleHLODISMCs.Num() - 1, ComponentToInstancesMapping.InstancesStart, ComponentToInstancesMapping.InstancesCount);
				}
			}
		}

		// Sort entries by ActorIndex -> ComponentIndex -> InstancingStart
		ISMActorMapping.Sort([](const FISMActorMapping& A, const FISMActorMapping& B)
		{
			if (A.ActorIndex != B.ActorIndex)
			{
				return A.ActorIndex < B.ActorIndex;
			}

			if (A.ComponentIndex != B.ComponentIndex)
			{
				return A.ComponentIndex < B.ComponentIndex;
			}

			check(A.InstancingStart != B.InstancingStart);
			return A.InstancingStart < B.InstancingStart;
		});

		// Fold entries with the same ActorIndex and ComponentIndex, and adjust the InstancingStart & InstancingCount
		if (!ISMActorMapping.IsEmpty())
		{
			int32 Write = 0; // index of the current output slot

			for (int32 Read = 1; Read < ISMActorMapping.Num(); ++Read)
			{
				FISMActorMapping& Current = ISMActorMapping[Write];
				const FISMActorMapping& Next = ISMActorMapping[Read];

				const bool bSameActorAndComp =
					(Current.ActorIndex == Next.ActorIndex) &&
					(Current.ComponentIndex == Next.ComponentIndex);

				// "Consecutive" means Next begins exactly where Cur ends.
				const bool bConsecutive =
					bSameActorAndComp &&
					(Next.InstancingStart == Current.InstancingStart + Current.InstancingCount);

				if (bConsecutive)
				{
					// Extend current run
					Current.InstancingCount += Next.InstancingCount;
				}
				else
				{
					// Start a new run
					++Write;
					if (Write != Read)
					{
						ISMActorMapping[Write] = Next;
					}
				}
			}

			// Trim the array to only contain the elements we've written
			ISMActorMapping.SetNum(Write + 1, EAllowShrinking::No);
		}

		PackedISMData.ISMCs = DestructibleHLODISMCs;

		// Build the optimal representation of those mapping into our FHLODInstancingPackedMappingData structure
		int32 CurrentIndex = 0;
		while (CurrentIndex < ISMActorMapping.Num())
		{
			uint32 CurrentActorIndex = ISMActorMapping[CurrentIndex].ActorIndex;
			uint32 ConsecutiveCount = 1;
			while (CurrentIndex + ConsecutiveCount < (uint32)ISMActorMapping.Num() && ISMActorMapping[CurrentIndex + ConsecutiveCount].ActorIndex == CurrentActorIndex)
			{
				ConsecutiveCount++;
			}

			// If we have a single component for that actor, we can store the mapping inline in the FActorInstanceMappingsRef structure
			if (ConsecutiveCount == 1)
			{
				const FISMActorMapping& Mapping = ISMActorMapping[CurrentIndex];
				PackedISMData.PerActorMappingData.Emplace(CurrentActorIndex, FActorInstanceMappingsRef::MakeMappingInline(Mapping.ComponentIndex, Mapping.InstancingStart, Mapping.InstancingCount));
			}
			else
			{
				// We have multiple entries, data will be stored in multiple FComponentInstanceMapping entries
				PackedISMData.PerActorMappingData.Emplace(CurrentActorIndex, FActorInstanceMappingsRef::MakeMappingRange(PackedISMData.ComponentsMapping.Num(), ConsecutiveCount));

				for (uint32 ComponentMappingIndex = 0; ComponentMappingIndex < ConsecutiveCount; ComponentMappingIndex++)
				{
					const FISMActorMapping& ComponentMapping = ISMActorMapping[CurrentIndex + ComponentMappingIndex];
					PackedISMData.ComponentsMapping.Emplace(FComponentInstanceMapping::Make(ComponentMapping.ComponentIndex, ComponentMapping.InstancingStart, ComponentMapping.InstancingCount));
				}
			}

			CurrentIndex += ConsecutiveCount;
		}
	}

	if (!DestructibleActors.IsEmpty())
	{
		UWorldPartitionDestructibleHLODComponent* DestructibleHLODComponent = NewObject<UWorldPartitionDestructibleHLODComponent>();
		DestructibleHLODComponent->SetIsReplicated(true);
		DestructibleHLODComponent->SetWorldLocation(CachedWorldPosition);
		DestructibleHLODComponent->SetDestructibleActors(DestructibleActors);
		DestructibleHLODComponent->SetDestructibleHLODMaterial(DestructibleMaterial);
		DestructibleHLODComponent->SetHLODInstancingPackedMappingData(MoveTemp(PackedISMData));
		InOutComponents.Add(DestructibleHLODComponent);
	}

	DestructionMeshMergeExtension.Reset();
}

bool UWorldPartitionHLODModifierMeshDestruction::MustBeLoadedOnServer() const
{
	// As HLOD destruction needs to be replicated, HLOD actors which supports destruction must be loaded on the server.
	return true;
}