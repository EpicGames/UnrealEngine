// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/RebuildPolicies/HLODRebuildPolicyHashCompare.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODInstancedStaticMeshComponent.h"
#include "WorldPartition/HLOD/HLODSourceActors.h"

UHLODRebuildPolicyData* UHLODRebuildPolicyHashCompare::ComputeDataForRebuildPolicy(const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents, const UHLODRebuildPolicyData* InExistingPolicyData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHLODRebuildPolicyHashCompare::ComputeDataForRebuildPolicy);

	FHLODHashBuilder HashBuilder;
	ComputeHLODHash(HashBuilder, InHLODActor, InSourceComponents);

	UHLODRebuildPolicyHashData* HashData = NewObject<UHLODRebuildPolicyHashData>();
	HashData->HLODHash = HashBuilder.GetCrc();
	HashData->HLODHashReport = HashBuilder.BuildHashReport();

	return HashData;
}

EHLODRebuildPolicyDecision UHLODRebuildPolicyHashCompare::Evaluate(const AWorldPartitionHLOD* InHLODActor, const UHLODRebuildPolicyData* InOldData, const UHLODRebuildPolicyData* InNewData, FString& OutReason) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHLODRebuildPolicyHashCompare::Evaluate);

	const UHLODRebuildPolicyHashData* OldData = Cast<UHLODRebuildPolicyHashData>(InOldData);
	const UHLODRebuildPolicyHashData* NewData = CastChecked<UHLODRebuildPolicyHashData>(InNewData);

	// Hash used to be stored directly in the HLOD actor, so retrieve it from there when necessary
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const uint32 OldHash = OldData ? OldData->HLODHash : InHLODActor->GetHLODHash();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	const uint32 NewHash = NewData->HLODHash;

	return OldHash != NewHash ? EHLODRebuildPolicyDecision::ApproveRebuild : EHLODRebuildPolicyDecision::RejectRebuild;
}

void UHLODRebuildPolicyHashCompare::ComputeHLODHash(FHLODHashBuilder& InHashBuilder, const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents) const
{
	// Base key, changing this will force a rebuild of all HLODs
	FString HLODBaseKey = TEXT("0D33837AB1A04CC2AEEB04C5C217DE96");
	InHashBuilder.HashField(HLODBaseKey, TEXT("HLODBaseKey"));

	// HLOD Source Actors
	if (ensure(InHLODActor->GetSourceActors()))
	{
		InHLODActor->GetSourceActors()->ComputeHLODHash(InHashBuilder);
	}

	// Min Visible Distance
	InHashBuilder.HashField(InHLODActor->GetMinVisibleDistance(), TEXT("MinVisibleDistance"));

	// ISM Component Class
	TSubclassOf<UHLODInstancedStaticMeshComponent> HLODISMComponentClass = UHLODBuilder::GetInstancedStaticMeshComponentClass();
	if (HLODISMComponentClass != UHLODInstancedStaticMeshComponent::StaticClass())
	{
		InHashBuilder.HashField(HLODISMComponentClass->GetPathName(), TEXT("HLODISMComponentClass"));
	}

	// Append all components hashes
	UHLODBuilder::ComputeHLODHash(InHashBuilder, InSourceComponents);
}
