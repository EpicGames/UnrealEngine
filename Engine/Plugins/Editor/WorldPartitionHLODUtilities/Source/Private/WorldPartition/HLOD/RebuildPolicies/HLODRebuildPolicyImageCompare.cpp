// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/RebuildPolicies/HLODRebuildPolicyImageCompare.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODSourceActors.h"

#include "AssetCompilingManager.h"
#include "Async/ParallelFor.h"
#include "Components/PrimitiveComponent.h"
#include "ContentStreaming.h"
#include "EngineModule.h"
#include "Engine/Canvas.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "ImageComparer.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Interfaces/IScreenShotComparisonModule.h"
#include "Interfaces/IScreenShotManager.h"
#include "Interfaces/IScreenShotToolsModule.h"
#include "JsonObjectConverter.h"
#include "LegacyScreenPercentageDriver.h"
#include "MaterialShared.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstance.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ObjectCacheContext.h"
#include "RenderingThread.h"
#include "RHIGPUReadback.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "TextureResource.h"
#include "Widgets/SWindow.h"

static bool GWorldPartitionHLODRebuildPolicyImageCompareDebugImageDiff = false;
static FAutoConsoleVariableRef CVarWorldPartitionHLODRebuildPolicyImageCompareDebugImageDiff(
	TEXT("wp.Editor.HLOD.RebuildPolicy.ImageCompare.DebugImageDiff"),
	GWorldPartitionHLODRebuildPolicyImageCompareDebugImageDiff,
	TEXT("Display comparison results in an editor window"),
	ECVF_Scalability);

struct FSceneCaptureConfig
{
	uint8					BytesPerPixel;
	const TCHAR*			CaptureName;
	EViewModeIndex			ViewModeIndex;
	FName					VisualizationMode;
};

const FSceneCaptureConfig CaptureConfigs[EHLODCaptureImageType::Num] =
{
	{ 1,	TEXT("Alpha"),		VMI_Lit,				NAME_None },
	{ 3,	TEXT("BaseColor"),	VMI_VisualizeBuffer,	TEXT("BaseColor") },
	{ 3,	TEXT("Normal"),		VMI_VisualizeBuffer,	TEXT("WorldNormal") },
	{ 3,	TEXT("Emissive"),	VMI_VisualizeBuffer,	TEXT("PreTonemapHDRColor") },
	{ 1,	TEXT("Metallic"),	VMI_VisualizeBuffer,	TEXT("Metallic") },
	{ 1,	TEXT("Roughness"),	VMI_VisualizeBuffer,	TEXT("Roughness") },
	{ 1,	TEXT("Specular"),	VMI_VisualizeBuffer,	TEXT("Specular") },
};

const FRotator CaptureAngles[] =
{
	// Zenith (straight down)
	FRotator(-90.f,   0.f,   0.f),

	// Ring A @ elevation 40°
	FRotator(-40.f,   0.f,   0.f),
	FRotator(-40.f,  90.f,   0.f),
	FRotator(-40.f, 180.f,   0.f),
	FRotator(-40.f, 270.f,   0.f),

	// Ring B @ elevation 10°
	FRotator(-10.f,  45.f,   0.f),
	FRotator(-10.f, 135.f,   0.f),
	FRotator(-10.f, 225.f,   0.f),
	FRotator(-10.f, 315.f,   0.f)
};

static FMatrix BuildOrthoMatrix(float InOrthoWidth, float InOrthoHeight)
{
	const double ZOffset = UE_FLOAT_HUGE_DISTANCE / 2.0;
	return FReversedZOrthoMatrix(
		InOrthoWidth / 2.0f,
		InOrthoHeight / 2.0f,
		0.5f / ZOffset,
		ZOffset
	);
}

static uint32 GetScreenSize(const FBoxSphereBounds& InBounds, double InExpectedDrawDistance)
{
	// Generate a projection matrix.
	static const float ScreenX = 1920;
	static const float ScreenY = 1080;
	static const float HalfFOVRad = FMath::DegreesToRadians(45.0f);
	static const FMatrix ProjectionMatrix = FPerspectiveMatrix(HalfFOVRad, ScreenX, ScreenY, 0.01);

	float ScreenSizePercent = ComputeBoundsScreenSize(FVector::ZeroVector, (float)InBounds.SphereRadius, FVector(0.0, 0.0, InExpectedDrawDistance), ProjectionMatrix);
	float ScreenSizePixel = ScreenSizePercent * ScreenX;
	return (uint32)FMath::CeilToInt32(ScreenSizePixel);
}

static void PerformSceneWarmup(UWorld* InCaptureWorld, const TArray<UActorComponent*>& InSourceComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Internal::PerformSceneWarmup);

	// Wait for pending load requests.
	FlushAsyncLoading();

	// Inspired from UMaterialInterface::SubmitRemainingJobsForWorld() 
	// Modified to handle MaterialInstance parent materials
	{
		TSet<UMaterialInterface*> MaterialsToCache;
		FObjectCacheContextScope ObjectCacheScope;

		for (UActorComponent* HLODRelevantComponent : InSourceComponents)
		{
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(HLODRelevantComponent))
			{
				TObjectCacheIterator<UMaterialInterface> UsedMaterials = ObjectCacheScope.GetContext().GetUsedMaterials(PrimitiveComponent->GetPrimitiveComponentInterface());
				for (UMaterialInterface* MaterialInterface : UsedMaterials)
				{
					while (MaterialInterface && !MaterialsToCache.Contains(MaterialInterface))
					{
						if (!MaterialInterface->IsComplete())
						{
							MaterialsToCache.Add(MaterialInterface);
						}

						UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
						MaterialInterface = MaterialInstance ? MaterialInstance->Parent.Get() : nullptr;
					};
				}
			}
		}

		if (MaterialsToCache.Num())
		{
			FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::SyncWithRenderingThread);
			for (UMaterialInterface* Material : MaterialsToCache)
			{
				// This is needed because CacheShaders blindly recreates uniform buffers
				// which can only be done if the draw command is going to be re-cached.
				UpdateContext.AddMaterialInterface(Material);
				Material->CacheShaders(EMaterialShaderPrecompileMode::Default);
			}
		}
	}

	// Wait for shader and other asset compilation to finish.
	FAssetCompilingManager::Get().FinishAllCompilation();

	// Force all mips to load.
	UTexture::ForceUpdateTextureStreaming();

	// Force all streamed resources to finish.
	IStreamingManager::Get().StreamAllResources(0.0f);

	// Ensure all deferred construction scripts are executed
	FAssetCompilingManager::Get().ProcessAsyncTasks();

	InCaptureWorld->SendAllEndOfFrameUpdates();
}

static void PerformSceneRender(FCanvas* InCanvas, FSceneViewFamily* InViewFamily)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Internal::PerformSceneRender);

	GFrameCounter++;

	ENQUEUE_RENDER_COMMAND(BeginFrameCommand)(
		[CurrentFrameCounter = GFrameCounter](FRHICommandListImmediate& RHICmdList)
		{
			GFrameCounterRenderThread = CurrentFrameCounter;
			GFrameNumberRenderThread++;
			FCoreDelegates::OnBeginFrameRT.Broadcast();
		});

	GetRendererModule().BeginRenderingViewFamily(InCanvas, InViewFamily);
	
	ENQUEUE_RENDER_COMMAND(EndFrameCommand)(
		[](FRHICommandListImmediate& RHICmdList)
		{
			FCoreDelegates::OnEndFrameRT.Broadcast();
			RHICmdList.EndFrame();
		});
}

static void PerformViewWarmup(FCanvas* InCanvas, FSceneViewFamily* InViewFamily, uint32 InWarmupFrameCount)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Internal::PerformViewWarmup);

	bool bCompilingAssets = false;
	uint32 NumRender = 0;
	
	do
	{
		PerformSceneRender(InCanvas, InViewFamily);

		NumRender++;

		// Flush rendering commands, may queue assets compilation (shaders)
		FlushRenderingCommands();

		// If there are assets to be compiled, we need to wait for compilation to finish and start render over
		bCompilingAssets = FAssetCompilingManager::Get().GetNumRemainingAssets() > 0;
		if (bCompilingAssets)
		{
			FAssetCompilingManager::Get().FinishAllCompilation();

			// Ensure all deferred construction scripts are executed
			FAssetCompilingManager::Get().ProcessAsyncTasks();
		}
	} while (bCompilingAssets || NumRender < InWarmupFrameCount);
}

static TSharedPtr<FRHIGPUTextureReadback> RenderSceneToTexture(FSceneInterface* InScene, const bool bInPerformWarmup, uint32 InWarmupFrameCount, EHLODCaptureImageType InCaptureType, const FVector& InViewOrigin, const FRotator& InViewRotation, const FMatrix& InProjectionMatrix, const TSet<FPrimitiveComponentId>& InVisibleComponents, UTextureRenderTarget2D* InRenderTargetTexture)
{
	const FSceneCaptureConfig& CaptureConfig = CaptureConfigs[InCaptureType];

	FEngineShowFlags ShowFlags(ESFIM_Game);
	ApplyViewMode(CaptureConfig.ViewModeIndex, /*bPerspective=*/false, ShowFlags);
	ShowFlags.DisableAdvancedFeatures();
	ShowFlags.SetAmbientCubemap(false);
	ShowFlags.SetAmbientOcclusion(false);
	ShowFlags.SetAntiAliasing(true);
	ShowFlags.SetTemporalAA(false);
	ShowFlags.SetAtmosphere(false);
	ShowFlags.SetBloom(false);
	ShowFlags.SetCameraImperfections(false);
	ShowFlags.SetCapsuleShadows(false);
	ShowFlags.SetCloud(false);
	ShowFlags.SetColorGrading(false);
	ShowFlags.SetContactShadows(false);
	ShowFlags.SetDepthOfField(false);
	ShowFlags.SetDistanceFieldAO(false);
	ShowFlags.SetDynamicShadows(false);
	ShowFlags.SetEyeAdaptation(false);
	ShowFlags.SetFog(false);
	ShowFlags.SetGlobalIllumination(false);
	ShowFlags.SetGrain(false);
	ShowFlags.SetHighResScreenshotMask(false);
	ShowFlags.SetHMDDistortion(false);
	ShowFlags.SetIndirectLightingCache(false);
	ShowFlags.SetLightShafts(false);
	ShowFlags.SetLensFlares(false);
	ShowFlags.SetMotionBlur(false);
	ShowFlags.SetOnScreenDebug(false);
	ShowFlags.SetPostProcessing(true);
	ShowFlags.SetReflectionEnvironment(false);
	ShowFlags.SetScreenPercentage(false);
	ShowFlags.SetScreenSpaceReflections(false);
	ShowFlags.SetSeparateTranslucency(false);
	ShowFlags.SetSkyLighting(false);
	ShowFlags.SetStereoRendering(false);
	ShowFlags.SetTonemapper(false);
	ShowFlags.SetToneCurve(false);
	ShowFlags.SetVignette(false);
	ShowFlags.SetVolumetricFog(false);
	ShowFlags.SetVolumetricLightmap(false);	
	
	ShowFlags.SetVisualizeBuffer(CaptureConfig.ViewModeIndex == VMI_VisualizeBuffer);
	ShowFlags.SetPostProcessMaterial(CaptureConfig.ViewModeIndex == VMI_VisualizeBuffer);

	ShowFlags.SetLighting(InCaptureType == EHLODCaptureImageType::Capture_Emissive);
	
	// Use a constant (0) game time to enforce scene determinism between captures
	static const FGameTime CaptureTime;

	FTextureRenderTargetResource* RenderTargetResource = InRenderTargetTexture->GameThread_GetRenderTargetResource();

	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(RenderTargetResource, InScene, ShowFlags)
		.SetTime(CaptureTime)
		.SetRealtimeUpdate(false)
	);
	ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(ViewFamily, 1.0f));

	static const FMatrix ViewPlanesMatrix = FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.SetViewRectangle(FIntRect(0, 0, InRenderTargetTexture->SizeX, InRenderTargetTexture->SizeY));
	ViewInitOptions.BackgroundColor = FLinearColor(FColor(0, 0, 0, 0));
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.ViewOrigin = InViewOrigin;
	ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(InViewRotation) * ViewPlanesMatrix;
	ViewInitOptions.ProjectionMatrix = InProjectionMatrix;

	if (InVisibleComponents.Num() > 0)
	{
		ViewInitOptions.ShowOnlyPrimitives = InVisibleComponents;
	}

	FSceneView* NewView = new FSceneView(ViewInitOptions);
	NewView->CurrentBufferVisualizationMode = CaptureConfig.VisualizationMode;
	ViewFamily.Views.Add(NewView);

	FCanvas Canvas(RenderTargetResource, NULL, CaptureTime, InScene->GetFeatureLevel());

	if (bInPerformWarmup)
	{
		PerformViewWarmup(&Canvas, &ViewFamily, InWarmupFrameCount);
	}

	PerformSceneRender(&Canvas, &ViewFamily);

	TSharedPtr<FRHIGPUTextureReadback> ReadbackRequest = MakeShared<FRHIGPUTextureReadback>(TEXT("HLODSceneCaptureReadback"));

	ENQUEUE_RENDER_COMMAND(HLODSceneCapture)([ReadbackRequest = ReadbackRequest, RenderTargetResource](FRHICommandListImmediate& RHICmdList) mutable
	{
		ReadbackRequest->EnqueueCopy(RHICmdList, RenderTargetResource->GetRenderTargetTexture());
	});

	return ReadbackRequest;
}

static void SavePNG(const TArray<FColor>& InPixels, int32 InImageWidth, int32 InImageHeight, const FString& InFilename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SavePNG);

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

	if (ImageWrapper.IsValid() && ImageWrapper->SetRaw(InPixels.GetData(), InPixels.NumBytes(), InImageWidth, InImageHeight, ERGBFormat::BGRA, 8))
	{
		TArray64<uint8> RawData = ImageWrapper->GetCompressed();
		FFileHelper::SaveArrayToFile(RawData, *InFilename);
	}
}


static bool AreBoxesWithinPercent(const FBox& InCandidate, const FBox& InReference, double InPercent)
{
	const double Alpha = FMath::Clamp(InPercent, 0.f, 1.f);

	FVector RefSize = InReference.GetSize();
	RefSize.X = FMath::Max(RefSize.X, KINDA_SMALL_NUMBER);
	RefSize.Y = FMath::Max(RefSize.Y, KINDA_SMALL_NUMBER);
	RefSize.Z = FMath::Max(RefSize.Z, KINDA_SMALL_NUMBER);

	const double WorstMin = ((InCandidate.Min - InReference.Min).GetAbs() / RefSize).GetMax();
	const double WorstMax = ((InCandidate.Max - InReference.Max).GetAbs() / RefSize).GetMax();

	return FMath::Max(WorstMin, WorstMax) <= Alpha;
}


UHLODRebuildPolicyImageCompare::UHLODRebuildPolicyImageCompare(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UHLODRebuildPolicyData* UHLODRebuildPolicyImageCompare::ComputeDataForRebuildPolicy(const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents, const UHLODRebuildPolicyData* InExistingPolicyData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHLODRebuildPolicyImageCompare::ComputeDataForRebuildPolicy);

	if (InSourceComponents.IsEmpty())
	{
		return nullptr;
	}

	// Propagate alpha to render texture during scene capture
	// Used to obtain an alpha mask of what is actually rendered to the scene
	static IConsoleVariable* CVarPropagateAlpha = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha"));
	CVarPropagateAlpha->Set(true, EConsoleVariableFlags(ECVF_SetByCode | ECVF_Set_NoSinkCall_Unsafe));
	ON_SCOPE_EXIT
	{
		CVarPropagateAlpha->Unset(EConsoleVariableFlags(ECVF_SetByCode | ECVF_Set_NoSinkCall_Unsafe));
	};

	UWorld* CaptureWorld = InSourceComponents[0]->GetWorld();
	
	PerformSceneWarmup(CaptureWorld, InSourceComponents);

	FBoxSphereBounds::Builder BoundsBuilder;
	TSet<FPrimitiveComponentId> VisibleComponents;
	for (UActorComponent* HLODRelevantComponent : InSourceComponents)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(HLODRelevantComponent))
		{
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SceneComponent))
			{
				BoundsBuilder += PrimitiveComponent->GetBounds().GetBox();
				VisibleComponents.Add(PrimitiveComponent->GetPrimitiveSceneId());
			}
		}
	}
	FBox TargetBounds = FBoxSphereBounds(BoundsBuilder).GetBox();

	const UHLODRebuildPolicyImageData* ExistingPolicyData = Cast<UHLODRebuildPolicyImageData>(InExistingPolicyData);
	if (ExistingPolicyData)
	{
		if (AreBoxesWithinPercent(TargetBounds, ExistingPolicyData->CaptureBounds, CaptureImageMargin))
		{
			TargetBounds = ExistingPolicyData->CaptureBounds;
		}
	}

	FVector CameraTarget = TargetBounds.GetCenter();
	double CaptureDistance = TargetBounds.GetExtent().GetMin() + InHLODActor->GetMinVisibleDistance();

	UHLODRebuildPolicyImageData* ImageData = NewObject<UHLODRebuildPolicyImageData>();
	ImageData->CaptureBounds = TargetBounds;
	ImageData->CaptureFrames.Reserve(UE_ARRAY_COUNT(CaptureAngles));

	const uint32 MinImageSize = 16;
	uint32 ImageSize = FMath::Clamp(GetScreenSize(TargetBounds, CaptureDistance), MinImageSize, MaxImageSize);
	
	// Flag kept to ensure we perform warmup of the scene only once
	bool bPerformedWarmup = false;

	// Struct used to enqueue and process readback operations
	struct FPendingReadback
	{
		uint32 AngleIndex;
		uint32 ImageIndex;
		UTextureRenderTarget2D* RenderTargetTexture;
		TSharedPtr<FRHIGPUTextureReadback> TextureReadback;
		TArray<FColor>& Pixels;
	};

	// Perform scene capture from multiple angles
	for (uint32 AngleIndex = 0; AngleIndex < UE_ARRAY_COUNT(CaptureAngles); ++AngleIndex)
	{
		const FRotator& CaptureAngle = CaptureAngles[AngleIndex];
	
		FHLODCaptureFrame& CaptureFrame = ImageData->CaptureFrames.Emplace_GetRef();

		CaptureFrame.CameraLocation = CameraTarget + CaptureAngle.Vector() * CaptureDistance;
		CaptureFrame.CameraRotation = CaptureAngle;
		CaptureFrame.OrthoWidth = (float)TargetBounds.GetSize().Length() * (1 + CaptureImageMargin);

		// For each camera angle, capture different GBuffer properties (base color, normal, metallic, roughnes, etc.)
		for (uint32 CaptureIndex = 0; CaptureIndex < EHLODCaptureImageType::Num; ++CaptureIndex)
		{
			EHLODCaptureImageType CaptureType = (EHLODCaptureImageType)CaptureIndex;

			FMatrix ProjectionMatrix = BuildOrthoMatrix(CaptureFrame.OrthoWidth, CaptureFrame.OrthoWidth);

			UTextureRenderTarget2D* RenderTargetTexture = NewObject<UTextureRenderTarget2D>();
			RenderTargetTexture->InitCustomFormat(ImageSize, ImageSize, PF_B8G8R8A8, false);

			TSharedPtr<FRHIGPUTextureReadback> TextureReadback = RenderSceneToTexture(
				CaptureWorld->Scene,
				!bPerformedWarmup,
				WarmupFrameCount,
				CaptureType,
				CaptureFrame.CameraLocation,
				CaptureFrame.CameraRotation,
				ProjectionMatrix,
				VisibleComponents,
				RenderTargetTexture);

			FPendingReadback PendingReadback = { AngleIndex, CaptureType, RenderTargetTexture, TextureReadback, CaptureFrame.Images[CaptureType].Pixels };
			ENQUEUE_RENDER_COMMAND(PerformReadback)(
				[PendingReadback, ImageSize](FRHICommandListImmediate& RHICmdList) mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(DrainAllReadbacks);

					// Wait until the GPU copy finished.
					while (!PendingReadback.TextureReadback->IsReady())
					{
						RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
						FPlatformProcess::SleepNoStats(0.0f);
					}

					int32 RowPitchInPixels = 0;
					const void* Src = PendingReadback.TextureReadback->Lock(RowPitchInPixels);
					const int32 RowPitchInBytes = RowPitchInPixels * sizeof(FColor);
					const uint32 DstBytesPerRow = ImageSize * sizeof(FColor);

					PendingReadback.Pixels.SetNumUninitialized(ImageSize * ImageSize);
					uint8* Dst = reinterpret_cast<uint8*>(PendingReadback.Pixels.GetData());

					for (uint32 y = 0; y < ImageSize; ++y)
					{
						FMemory::Memcpy(Dst + y * DstBytesPerRow, static_cast<const uint8*>(Src) + y * RowPitchInBytes, DstBytesPerRow);
					}

					PendingReadback.TextureReadback->Unlock();
				});
		}

		bPerformedWarmup = true;
	}

	FlushRenderingCommands();

	// Finalize captured data
	// Alpha mask -> Fill RGB as 255,255,255. Captured alpha is actually the inverse, so we need to reverse it.
	// Others -> Copy alpha over from alpha mask
	ParallelFor(ImageData->CaptureFrames.Num(), [&](int32 iCaptureFrame)
	{
		FHLODCaptureFrame& CaptureFrame = ImageData->CaptureFrames[iCaptureFrame];

		for (int32 ImageIndex = 0; ImageIndex < EHLODCaptureImageType::Num; ++ImageIndex)
		{
			const FSceneCaptureConfig& CaptureConfig = CaptureConfigs[ImageIndex];

			FHLODCaptureImage& Image = CaptureFrame.Images[ImageIndex];
			Image.Width = ImageSize;
			Image.Height = ImageSize;
			Image.BytesPerPixel = CaptureConfig.BytesPerPixel;

			TArray<FColor>& Pixels = Image.Pixels;
			TArray<FColor>& PixelsAlpha = CaptureFrame.Images[Capture_AlphaMask].Pixels;

			if (ImageIndex == Capture_AlphaMask)
			{
				for (int32 iPixel = 0; iPixel < Pixels.Num(); ++iPixel)
				{
					Pixels[iPixel].A = 255 - Pixels[iPixel].A;
					Pixels[iPixel].R = 255;
					Pixels[iPixel].G = 255;
					Pixels[iPixel].B = 255;
				}
			}
			else
			{
				for (int32 iPixel = 0; iPixel < Pixels.Num(); ++iPixel)
				{
					Pixels[iPixel].A = PixelsAlpha[iPixel].A;
				}
			}
		}
	});

	FHLODHashBuilder HashBuilder;
	const UWorldPartitionHLODSourceActors* HLODSourceActors = InHLODActor->GetSourceActors();
	const UHLODLayer* HLODLayer = HLODSourceActors ? HLODSourceActors->GetHLODLayer() : nullptr;
	if (HLODLayer)
	{
		HLODLayer->ComputeHLODHash(HashBuilder);
		ImageData->HLODLayerHash = HashBuilder.GetCrc();
	}
	
	ImageData->bNeedsDataCompression = true;
	ImageData->bNeedsDataDecompression = false;

	return ImageData;
}

template<bool bSingleChannel, typename SetErrorFunc>
void CompareStructuralSimilarity(int32 InImageWidth, int32 InImageHeight, const TArray<FColor>& InPixelsA, const TArray<FColor>& InPixelsB, SetErrorFunc&& InSetErrorFunc, float& OutMaxLocalDifference, float& OutGlobalDifference)
{
	// Implementation of https://en.wikipedia.org/wiki/Structural_similarity

	static const int32 BitsPerComponent = 8;
	static const double K1 = 0.01;
	static const double K2 = 0.03;
	static const double L = (1 << BitsPerComponent) - 1;
	static const double C1 = FMath::Pow(K1 * L, 2);
	static const double C2 = FMath::Pow(K2 * L, 2);

	double WeightedSSIM = 0.0;
	double MaxWindowDSSIM = 0;
	
	// Global alpha-weighted total
	double WeightedTotalAlpha = 0.0;
	
	// Gaussian kernel (11x11, sigma=1.5)
	// Static, computed once
	constexpr int32 KernelRadius = 5;
	constexpr int32 KernelSize = KernelRadius * 2 + 1;
	static const TStaticArray<double, KernelSize> Kernel = [] 
	{
		TStaticArray<double, KernelSize> K{};
		const double Sigma = 1.5;
		const double TwoSigma2 = 2.0 * Sigma * Sigma;
		double Sum = 0.0;
		for (int i = -KernelRadius; i <= KernelRadius; ++i)
		{
			const double Value = FMath::Exp(-(double)(i * i) / TwoSigma2);
			K[i + KernelRadius] = Value;
			Sum += Value;
		}
		for (int i = 0; i < KernelSize; ++i)
		{
			K[i] /= Sum;
		}
		return K;
	}();

	// Index directly in memory rather than going through the TArray operator[], as
	// it adds quite a bit of overhead in repeated accesses like the kernel processing below.
	const FColor* PixelsA = InPixelsA.GetData();
	const FColor* PixelsB = InPixelsB.GetData();

	for (int32 X = 0; X < InImageWidth; ++X)
	{
		for (int32 Y = 0; Y < InImageHeight; ++Y)
		{
			// Per-window accumulators
			double SumW = 0.0;
			double SumA_R = 0.0, SumB_R = 0.0, SumAA_R = 0.0, SumBB_R = 0.0, SumAB_R = 0.0;
			double SumA_G = 0.0, SumB_G = 0.0, SumAA_G = 0.0, SumBB_G = 0.0, SumAB_G = 0.0;
			double SumA_B = 0.0, SumB_B = 0.0, SumAA_B = 0.0, SumBB_B = 0.0, SumAB_B = 0.0;

			for (int dy = -KernelRadius; dy <= KernelRadius; ++dy)
			{
				const int32 yy = FMath::Clamp(Y + dy, 0, InImageHeight - 1);
				const double WeightY = Kernel[dy + KernelRadius];

				for (int dx = -KernelRadius; dx <= KernelRadius; ++dx)
				{
					const int32 xx = FMath::Clamp(X + dx, 0, InImageWidth - 1);
					const double Weight = WeightY * Kernel[dx + KernelRadius];

					// Per-channel alpha-weighted accumulators (combine kernel with alpha)
					const FColor& CA = PixelsA[xx + yy * InImageWidth];
					const FColor& CB = PixelsB[xx + yy * InImageWidth];
					const double AlphaA = CA.A / 255.0;
					const double AlphaB = CB.A / 255.0;
					const double WeightedAlpha = (AlphaA + AlphaB) - (AlphaA * AlphaB);
					if (WeightedAlpha > 0.0)
					{
						const double WA = Weight * WeightedAlpha;
						SumW += WA;

						const double Ar = (double)CA.R;
						const double Br = (double)CB.R;
						SumA_R += WA * Ar;  SumB_R += WA * Br;  SumAA_R += WA * Ar * Ar;  SumBB_R += WA * Br * Br;  SumAB_R += WA * Ar * Br;

						if (!bSingleChannel)
						{
							const double Ag = (double)CA.G;
							const double Bg = (double)CB.G;
							SumA_G += WA * Ag;  SumB_G += WA * Bg;  SumAA_G += WA * Ag * Ag;  SumBB_G += WA * Bg * Bg;  SumAB_G += WA * Ag * Bg;

							const double Ab = (double)CA.B;
							const double Bb = (double)CB.B;
							SumA_B += WA * Ab;  SumB_B += WA * Bb;  SumAA_B += WA * Ab * Ab;  SumBB_B += WA * Bb * Bb;  SumAB_B += WA * Ab * Bb;
						}
					}
				}
			}

			double WindowSSIM = 1.0;
			double WindowDSSIM = 0.0;
			double WindowAlphaWeight = 0.0;

			// Skip if entire window is effectively transparent, in that case treat as perfect match with zero weight.
			if (SumW > 0.0)
			{
				auto ChannelSSIM = [&](double SumA, double SumB, double SumAA, double SumBB, double SumAB) -> double
				{
					const double AverageAChannel = SumA / SumW;
					const double AverageBChannel = SumB / SumW;

					double VarianceAChannel = SumAA / SumW - AverageAChannel * AverageAChannel;
					double VarianceBChannel = SumBB / SumW - AverageBChannel * AverageBChannel;
					double CovarianceABChannel = SumAB / SumW - AverageAChannel * AverageBChannel;

					VarianceAChannel = FMath::Max(0.0, VarianceAChannel);
					VarianceBChannel = FMath::Max(0.0, VarianceBChannel);

					const double LuminanceChannel = (2.0 * AverageAChannel * AverageBChannel + C1) / (AverageAChannel * AverageAChannel + AverageBChannel * AverageBChannel + C1);
					const double ContrastChannel = (2.0 * CovarianceABChannel + C2) / (VarianceAChannel + VarianceBChannel + C2);

					return LuminanceChannel * ContrastChannel;
				};

				const double SSIM_R = ChannelSSIM(SumA_R, SumB_R, SumAA_R, SumBB_R, SumAB_R);

				if (!bSingleChannel)
				{
					const double SSIM_G = ChannelSSIM(SumA_G, SumB_G, SumAA_G, SumBB_G, SumAB_G);
					const double SSIM_B = ChannelSSIM(SumA_B, SumB_B, SumAA_B, SumBB_B, SumAB_B);
					WindowSSIM = (SSIM_R + SSIM_G + SSIM_B) / 3.0;
				}
				else
				{
					WindowSSIM = SSIM_R;
				}

				// center pixel's weight
				const FColor& CA = PixelsA[X + Y * InImageWidth];
				const FColor& CB = PixelsB[X + Y * InImageWidth];
				const double AlphaA = CA.A / 255.0;
				const double AlphaB = CB.A / 255.0;
				const double WeightedAlpha = (AlphaA + AlphaB) - (AlphaA * AlphaB);
				WindowAlphaWeight = WeightedAlpha;
			}

			WindowSSIM = FMath::Clamp(WindowSSIM, -1.0, 1.0);
			WindowDSSIM = (1 - WindowSSIM) / 2;

			InSetErrorFunc(X, Y, WindowDSSIM);
		
			MaxWindowDSSIM = FMath::Max(MaxWindowDSSIM, WindowDSSIM);

			// Use alpha-weighted contribution
			WeightedSSIM += WindowSSIM * WindowAlphaWeight;
			WeightedTotalAlpha += WindowAlphaWeight;
		}
	}

	double SSIM = (WeightedTotalAlpha > 0.0) ? (WeightedSSIM / WeightedTotalAlpha) : 1.0;
	double DSSIM = (1.0 - SSIM) / 2.0;

	OutMaxLocalDifference = (float)FMath::Clamp(MaxWindowDSSIM, 0.0, 1.0);
	OutGlobalDifference = (float)FMath::Clamp(DSSIM, 0.0, 1.0);
}

struct FComparisonResult
{
	const FHLODCaptureImage*	ImageA;
	const FHLODCaptureImage*	ImageB;
	EHLODCaptureImageType		CaptureType;
	FString						ComparisonName;

	TArray<FColor> ErrorResult;
	float MaxLocalDSSIM = 0.0;
	float DSSIM = 0.0;
};

static FComparisonResult Compare(const FHLODCaptureFrame& InFrameA, const FHLODCaptureFrame& InFrameB, uint32 InCaptureFrameIdx, EHLODCaptureImageType InImageType, bool bInGatherErrorImage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CompareStructuralSimilarity);

	FComparisonResult ComparisonResult;
	ComparisonResult.ImageA = &InFrameA.Images[InImageType];
	ComparisonResult.ImageB = &InFrameB.Images[InImageType];
	ComparisonResult.CaptureType = InImageType;
	ComparisonResult.ComparisonName = FString::Printf(TEXT("Frame%d_%s"), InCaptureFrameIdx, CaptureConfigs[InImageType].CaptureName);

	check(ComparisonResult.ImageA->Width == ComparisonResult.ImageB->Width);
	check(ComparisonResult.ImageA->Height == ComparisonResult.ImageB->Height);

	if (bInGatherErrorImage)
	{
		ComparisonResult.ErrorResult.SetNumUninitialized(ComparisonResult.ImageA->Width * ComparisonResult.ImageA->Height);
	}

	auto SetError = [&](int32 x, int32 y, double WindowDSSIM)
	{
		if (bInGatherErrorImage)
		{
			uint8 QuantizedError = FColor::QuantizeUNormFloatTo8((float)WindowDSSIM);
			FColor ErrorColor(QuantizedError, QuantizedError, QuantizedError, 255);
			ComparisonResult.ErrorResult[y * ComparisonResult.ImageA->Width + x] = ErrorColor;
		}
	};

	const bool bSingleChannel = ComparisonResult.ImageA->BytesPerPixel == 1;
	if (bSingleChannel)
	{
		CompareStructuralSimilarity<true>(ComparisonResult.ImageA->Width, ComparisonResult.ImageA->Height, ComparisonResult.ImageA->Pixels, ComparisonResult.ImageB->Pixels, SetError, ComparisonResult.MaxLocalDSSIM, ComparisonResult.DSSIM);
	}
	else
	{
		CompareStructuralSimilarity<false>(ComparisonResult.ImageA->Width, ComparisonResult.ImageA->Height, ComparisonResult.ImageA->Pixels, ComparisonResult.ImageB->Pixels, SetError, ComparisonResult.MaxLocalDSSIM, ComparisonResult.DSSIM);
	}

	return ComparisonResult;
}

static void ExecuteDebugComparison(const TArray<FComparisonResult>& InComparisonResults, float MaximumLocalErrorTolerance, float MaximumGlobalErrorTolerance)
{
	const FString RunLabel = TEXT("HLOD_") + FDateTime::Now().ToString(TEXT("%Y-%m-%d_%H-%M-%S"));
	const FString ReportsRoot = FPaths::AutomationReportsDir();
	const FString ImportRoot = FPaths::Combine(ReportsRoot, RunLabel);
	const FString SubjectName = TEXT("HLOD");

	auto MakePathRelative = [&](const FString& Abs)
	{
		FString Rel = Abs;
		FPaths::MakePathRelativeTo(Rel, *ImportRoot);
		return Rel;
	};

	for (const FComparisonResult& ComparisonResult : InComparisonResults)
	{
		const FString PairName = ComparisonResult.ComparisonName;
		const FString ReportDir = FPaths::Combine(ImportRoot, PairName);
		const FString OldImagePath = FPaths::Combine(ReportDir, TEXT("Approved.png"));
		const FString NewImagePath = FPaths::Combine(ReportDir, TEXT("Incoming.png"));
		const FString DeltaPath = FPaths::Combine(ReportDir, TEXT("Delta.png"));
		const FString JsonPath = FPaths::Combine(ReportDir, TEXT("Report.json"));

		IFileManager::Get().MakeDirectory(*ReportDir, /*Tree=*/true);

		SavePNG(ComparisonResult.ImageA->Pixels, ComparisonResult.ImageA->Width, ComparisonResult.ImageA->Height, OldImagePath);
		SavePNG(ComparisonResult.ImageB->Pixels, ComparisonResult.ImageB->Width, ComparisonResult.ImageB->Height, NewImagePath);
		SavePNG(ComparisonResult.ErrorResult, ComparisonResult.ImageA->Width, ComparisonResult.ImageA->Height, DeltaPath);

		FImageComparisonResult Result;
		Result.ApprovedFilePath = OldImagePath;
		Result.ReportApprovedFilePath = MakePathRelative(OldImagePath);
		Result.IncomingFilePath = NewImagePath;
		Result.ReportIncomingFilePath = MakePathRelative(NewImagePath);
		Result.ComparisonFilePath = DeltaPath;
		Result.ReportComparisonFilePath = MakePathRelative(DeltaPath);
		
		Result.ScreenshotPath = PairName;
		Result.IdealApprovedFolderPath = FPaths::GetPath(OldImagePath);
		Result.CreationTime = FDateTime::Now();
		Result.bSkipAttachingImages = false;

		Result.MaxLocalDifference = ComparisonResult.MaxLocalDSSIM;
		Result.GlobalDifference = ComparisonResult.DSSIM;

		FImageTolerance ImageTolerance;
		ImageTolerance.IgnoreAntiAliasing = true;
		ImageTolerance.IgnoreColors = true;
		ImageTolerance.MaximumGlobalError = MaximumGlobalErrorTolerance;
		ImageTolerance.MaximumLocalError = MaximumLocalErrorTolerance;
		Result.Tolerance = ImageTolerance;

		FString Json;
		if (FJsonObjectConverter::UStructToJsonObjectString(Result, Json))
		{
			FFileHelper::SaveStringToFile(Json, *JsonPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
		}
	}

	IScreenShotToolsModule& Tools = FModuleManager::LoadModuleChecked<IScreenShotToolsModule>("ScreenShotComparisonTools");
	IScreenShotManagerPtr Manager = Tools.GetScreenShotManager();
	Manager->OpenComparisonReportsAsync(ImportRoot);

	IScreenShotComparisonModule& ComparisonUI = FModuleManager::LoadModuleChecked<IScreenShotComparisonModule>("ScreenShotComparison");

	TSharedRef<SWidget> Browser = ComparisonUI.CreateScreenShotComparison(Manager.ToSharedRef());

	TSharedRef<SWindow> Win = SNew(SWindow)
		.Title(FText::FromString(TEXT("Image Comparison")))
		.ClientSize(FVector2D(1200, 800))
		[Browser];

	FSlateApplication::Get().AddWindow(Win);
}


EHLODRebuildPolicyDecision UHLODRebuildPolicyImageCompare::Evaluate(const AWorldPartitionHLOD* InHLODActor, const UHLODRebuildPolicyData* InOldData, const UHLODRebuildPolicyData* InNewData, FString& OutReason) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHLODRebuildPolicyImageCompare::Evaluate);

	const UHLODRebuildPolicyImageData* OldData = Cast<UHLODRebuildPolicyImageData>(InOldData);
	const UHLODRebuildPolicyImageData* NewData = CastChecked<UHLODRebuildPolicyImageData>(InNewData);

	if (OldData == nullptr)
	{
		OutReason = TEXT("no old data to compare with");
		return EHLODRebuildPolicyDecision::None;
	}

	// == Special handling for the HLOD layer settings ==
	// As this policy is only comparing screen captures of the INPUT actors, it would not take
	// into account changes to the HLOD layer settings, as these can affect the OUTPUT of the
	// HLOD build.
	// The HLOD build should be allowed to be performed in such case.
	if (OldData->HLODLayerHash != NewData->HLODLayerHash)
	{
		OutReason = TEXT("hlod layer settings were modified");
		return EHLODRebuildPolicyDecision::ApproveRebuild;
	}

	if (OldData->CaptureFrames.Num() != NewData->CaptureFrames.Num())
	{
		OutReason = FString::Printf(TEXT("number of frames to compare differ, %d vs %d"), OldData->CaptureFrames.Num(), NewData->CaptureFrames.Num());
		return EHLODRebuildPolicyDecision::None;
	}

	for (int32 CaptureFrameIdx = 0; CaptureFrameIdx < OldData->CaptureFrames.Num(); ++CaptureFrameIdx)
	{
		const FHLODCaptureFrame& FrameA = OldData->CaptureFrames[CaptureFrameIdx];
		const FHLODCaptureFrame& FrameB = NewData->CaptureFrames[CaptureFrameIdx];
		
		for (int32 ImageIdx = 0; ImageIdx < EHLODCaptureImageType::Num; ++ImageIdx)
		{
			const FHLODCaptureImage& ImageA = FrameA.Images[ImageIdx];
			const FHLODCaptureImage& ImageB = FrameB.Images[ImageIdx];

			if ((ImageA.Width != ImageB.Width) || (ImageA.Height != ImageB.Height))
			{
				OutReason = FString::Printf(TEXT("image sizes differ, %dx%d vs %dx%d"), ImageA.Width, ImageA.Height, ImageB.Width, ImageB.Height);
				return EHLODRebuildPolicyDecision::None;
			}

			if (ImageA.BytesPerPixel != ImageB.BytesPerPixel)
			{
				OutReason = FString::Printf(TEXT("image bpp differ, %d vs %d"), ImageA.BytesPerPixel, ImageB.BytesPerPixel);
				return EHLODRebuildPolicyDecision::None;
			}
		}
	}

	// Ensure stored images are available in uncompressed state
	bool bDataValid = true;
	bDataValid &= OldData->DecompressData();
	bDataValid &= NewData->DecompressData();
	if (!bDataValid)
	{
		OutReason = TEXT("failed to decompress data");
		return EHLODRebuildPolicyDecision::None;
	}

	const int32 NumFrames = OldData->CaptureFrames.Num();
	const int32 NumComparisonPerFrame = (EHLODCaptureImageType::Num - 1); // -1 as we do not compare the alpha mask directly

	TArray<FComparisonResult> ComparisonResults;
	ComparisonResults.SetNum(NumFrames * NumComparisonPerFrame);

	ParallelFor(ComparisonResults.Num(), [&](int32 Idx)
	{
		int32 CaptureFrameIdx = Idx / NumComparisonPerFrame;
		const FHLODCaptureFrame& FrameA = OldData->CaptureFrames[CaptureFrameIdx];
		const FHLODCaptureFrame& FrameB = NewData->CaptureFrames[CaptureFrameIdx];

		EHLODCaptureImageType ImageIdx = (EHLODCaptureImageType)((Idx % NumComparisonPerFrame) + 1); // +1 as we do not compare the alpha mask directly
		
		ComparisonResults[Idx] = Compare(FrameA, FrameB, CaptureFrameIdx, ImageIdx, GWorldPartitionHLODRebuildPolicyImageCompareDebugImageDiff);
	});

	// If enabled, open up a comparison window
	if (GWorldPartitionHLODRebuildPolicyImageCompareDebugImageDiff)
	{
		ExecuteDebugComparison(ComparisonResults, MaximumLocalError, MaximumGlobalError);
	}

	double MaxLocalDiff = 0.0f;
	double MaxGlobalDiff = 0.0f;
	for (const FComparisonResult& ComparisonResult : ComparisonResults)
	{
		MaxLocalDiff = FMath::Max(MaxLocalDiff, ComparisonResult.MaxLocalDSSIM);
		MaxGlobalDiff = FMath::Max(MaxGlobalDiff, ComparisonResult.DSSIM);
	}

	if (MaxGlobalDiff > MaximumGlobalError)
	{
		OutReason = FString::Printf(TEXT("global diff: %.2f%% > thres: %.2f%%"), MaxGlobalDiff * 100, MaximumGlobalError * 100);
		return EHLODRebuildPolicyDecision::ApproveRebuild;
	}

	if (MaxLocalDiff > MaximumLocalError)
	{
		OutReason = FString::Printf(TEXT("local diff: %.2f%% > thres: %.2f%%"), MaxLocalDiff * 100, MaximumLocalError * 100);
		return EHLODRebuildPolicyDecision::ApproveRebuild;
	}

	OutReason = FString::Printf(TEXT("global diff: %.2f%% < thres: %.2f%%, local diff: %.2f%% < thres: %.2f%%"), MaxGlobalDiff * 100, MaximumGlobalError * 100, MaxLocalDiff * 100, MaximumLocalError * 100);
	return EHLODRebuildPolicyDecision::RejectRebuild;
}

void UHLODRebuildPolicyImageData::PreSave(FObjectPreSaveContext InObjectSaveContext)
{
	Super::PreSave(InObjectSaveContext);

	if (InObjectSaveContext.IsFromAutoSave())
	{
		return;
	}

	CompressData();
}

static const ANSICHAR	HLOD_IMAGE_COMPARISON_MAGIC_MARKER[] = "HLOD_IMAGE_COMPARISON_MAGIC_MARKER";
static const uint32		HLOD_IMAGE_COMPARISON_DATA_VERSION = 0;

void UHLODRebuildPolicyImageData::CompressData() const
{
	if (!bNeedsDataCompression)
	{
		return;
	}

	UHLODRebuildPolicyImageData& MutableThis = *const_cast<UHLODRebuildPolicyImageData*>(this);

	// Compress all the images and store them
	ParallelFor(CaptureFrames.Num(), [&](int32 AngleIndex)
	{
		FHLODCaptureFrame& CaptureFrame = MutableThis.CaptureFrames[AngleIndex];

		for (uint32 ImageIndex = 0; ImageIndex < EHLODCaptureImageType::Num; ++ImageIndex)
		{
			FHLODCaptureImage& CaptureImage = CaptureFrame.Images[ImageIndex];

			if (!CaptureImage.Pixels.IsEmpty())
			{
				uint8 DataVersion = HLOD_IMAGE_COMPARISON_DATA_VERSION;

				FAnsiString ImageTypeString(StaticEnum<EHLODCaptureImageType>()->GetNameStringByValue(ImageIndex));
				FAnsiString ImageName = FAnsiString::Printf("Angle%d_%s", AngleIndex, *ImageTypeString);
				int32 ImageNameLen = ImageName.Len();
				int32 UncompressedSize = CaptureImage.Pixels.Num() * CaptureImage.BytesPerPixel;

				TArray<uint8> UncompressedBuffer;
				UncompressedBuffer.SetNumUninitialized(UncompressedSize);

				if (ImageIndex == Capture_AlphaMask)
				{
					for (int32 iPixel = 0; iPixel < CaptureImage.Pixels.Num(); ++iPixel)
					{
						UncompressedBuffer[iPixel] = CaptureImage.Pixels[iPixel].A;
					}
				}
				else if (CaptureImage.BytesPerPixel == 1)
				{
					for (int32 iPixel = 0; iPixel < CaptureImage.Pixels.Num(); ++iPixel)
					{
						UncompressedBuffer[iPixel] = CaptureImage.Pixels[iPixel].R;
					}
				}
				else if (CaptureImage.BytesPerPixel == 3)
				{
					for (int32 iPixel = 0; iPixel < CaptureImage.Pixels.Num(); ++iPixel)
					{
						UncompressedBuffer[iPixel * 3 + 0] = CaptureImage.Pixels[iPixel].R;
						UncompressedBuffer[iPixel * 3 + 1] = CaptureImage.Pixels[iPixel].G;
						UncompressedBuffer[iPixel * 3 + 2] = CaptureImage.Pixels[iPixel].B;
					}
				}
				else
				{
					checkNoEntry();
				}

				TArray<uint8> CompressedBuffer;
				CompressedBuffer.SetNumUninitialized(UncompressedSize);
				int32 CompressedSize = UncompressedSize;
				FCompression::CompressMemory(NAME_Zlib, CompressedBuffer.GetData(), CompressedSize, UncompressedBuffer.GetData(), UncompressedSize, COMPRESS_BiasSize);
				CompressedBuffer.SetNum(CompressedSize);

				// Write all the info to a buffer that can be read easily outside of UE
				FMemoryWriter MemoryWriter(CaptureImage.PixelsCompressed);
				MemoryWriter.Serialize((void*)HLOD_IMAGE_COMPARISON_MAGIC_MARKER, sizeof(HLOD_IMAGE_COMPARISON_MAGIC_MARKER) - 1);
				MemoryWriter << DataVersion;
				MemoryWriter << ImageNameLen;
				MemoryWriter.Serialize((void*)*ImageName, ImageNameLen);
				MemoryWriter << CaptureImage.Width;
				MemoryWriter << CaptureImage.Height;
				MemoryWriter << CaptureImage.BytesPerPixel;
				MemoryWriter << UncompressedSize;
				MemoryWriter << CompressedBuffer;
			}
			else
			{
				CaptureImage.PixelsCompressed.Empty();
			}
		}
	});

	MutableThis.bNeedsDataCompression = false;
}

bool UHLODRebuildPolicyImageData::DecompressData() const
{
	if (!bNeedsDataDecompression)
	{
		return true;
	}

	UHLODRebuildPolicyImageData& MutableThis = *const_cast<UHLODRebuildPolicyImageData*>(this);

	TAtomic<uint32> ErrorCount(0);

	ParallelFor(CaptureFrames.Num(), [&](int32 AngleIndex)
	{
		FHLODCaptureFrame& CaptureFrame = MutableThis.CaptureFrames[AngleIndex];

		for (uint32 ImageIndex = 0; ImageIndex < EHLODCaptureImageType::Num; ++ImageIndex)
		{
			FHLODCaptureImage& CaptureImage = CaptureFrame.Images[ImageIndex];

			// Clear any stale data
			CaptureImage.Pixels.Reset();

			// Nothing to do?
			if (CaptureImage.PixelsCompressed.IsEmpty())
			{
				ErrorCount++;
				continue;
			}

			FMemoryReader Reader(CaptureImage.PixelsCompressed, /*bIsPersistent*/ true);

			// Validate header
			if (Reader.TotalSize() < sizeof(HLOD_IMAGE_COMPARISON_MAGIC_MARKER) - 1)
			{
				UE_LOGF(LogTemp, Warning, "HLOD image buffer too small (angle %d, image %d).", AngleIndex, ImageIndex);
				ErrorCount++;
				continue;
			}

			ANSICHAR Header[sizeof(HLOD_IMAGE_COMPARISON_MAGIC_MARKER)] = { 0 };
			Reader.Serialize(Header, sizeof(HLOD_IMAGE_COMPARISON_MAGIC_MARKER) - 1);

			if (FCStringAnsi::Strncmp(Header, HLOD_IMAGE_COMPARISON_MAGIC_MARKER, sizeof(HLOD_IMAGE_COMPARISON_MAGIC_MARKER) - 1) != 0)
			{
				UE_LOGF(LogTemp, Warning, "HLOD image header mismatch (angle %d, image %d).", AngleIndex, ImageIndex);
				ErrorCount++;
				continue;
			}

			// Version - Currently unused but might be useful to extend/modify the stored data in the future
			uint8 DataVersion = HLOD_IMAGE_COMPARISON_DATA_VERSION;
			Reader << DataVersion;

			// Image name (discarded after read; only used for external tooling)
			int32 ImageNameLen = 0;
			Reader << ImageNameLen;

			if (ImageNameLen < 0 || (int64)Reader.Tell() + ImageNameLen >(int64)Reader.TotalSize())
			{
				UE_LOGF(LogTemp, Warning, "Invalid image name length (angle %d, image %d).", AngleIndex, ImageIndex);
				ErrorCount++;
				continue;
			}

			if (ImageNameLen > 0)
			{
				TArray<ANSICHAR> NameBytes;
				NameBytes.SetNumUninitialized(ImageNameLen);
				Reader.Serialize(NameBytes.GetData(), ImageNameLen);
			}

			// Dimensions
			int32 Width;
			int32 Height;
			uint8 BytesPerPixel;
			Reader << Width;
			Reader << Height;
			Reader << BytesPerPixel;

			check(CaptureImage.Width == Width);
			check(CaptureImage.Height == Height);
			check(CaptureImage.BytesPerPixel == BytesPerPixel);

			// Uncompressed size
			int32 UncompressedSize = 0;
			Reader << UncompressedSize;

			// Compressed payload (serialized TArray<uint8>)
			TArray<uint8> CompressedBuffer;
			Reader << CompressedBuffer;

			// Inflate
			if (UncompressedSize <= 0 || CompressedBuffer.Num() <= 0)
			{
				UE_LOGF(LogTemp, Warning, "Invalid sizes while decompressing HLOD image (angle %d, image %d).", AngleIndex, ImageIndex);
				ErrorCount++;
				continue;
			}

			TArray<uint8> UncompressedBuffer;
			UncompressedBuffer.SetNumUninitialized(UncompressedSize);
			int32 OutSize = UncompressedSize;

			const bool bOk = FCompression::UncompressMemory(NAME_Zlib, UncompressedBuffer.GetData(), OutSize, CompressedBuffer.GetData(),CompressedBuffer.Num());
			if (!bOk || OutSize != UncompressedSize)
			{
				UE_LOGF(LogTemp, Warning, "Failed to uncompress HLOD image (angle %d, image %d).", AngleIndex, ImageIndex);
				CaptureImage.Pixels.Empty();
				ErrorCount++;
				continue;
			}

			CaptureImage.Pixels.SetNumUninitialized(Width * Height);

			if (ImageIndex == Capture_AlphaMask)
			{
				for (int32 iPixel = 0; iPixel < CaptureImage.Pixels.Num(); ++iPixel)
				{
					CaptureImage.Pixels[iPixel] = FColor(255, 255, 255, UncompressedBuffer[iPixel]);
				}
			}
			else if (CaptureImage.BytesPerPixel == 1)
			{
				for (int32 iPixel = 0; iPixel < CaptureImage.Pixels.Num(); ++iPixel)
				{
					const uint8 R = UncompressedBuffer[iPixel];
					const uint8 A = CaptureFrame.Images[Capture_AlphaMask].Pixels[iPixel].A; // Safe to dereference as it's the first image processed
					CaptureImage.Pixels[iPixel] = FColor(R, R, R, A);
				}
			}
			else if (CaptureImage.BytesPerPixel == 3)
			{
				for (int32 iPixel = 0; iPixel < CaptureImage.Pixels.Num(); ++iPixel)
				{
					const uint8 R = UncompressedBuffer[iPixel * CaptureImage.BytesPerPixel + 0];
					const uint8 G = UncompressedBuffer[iPixel * CaptureImage.BytesPerPixel + 1];
					const uint8 B = UncompressedBuffer[iPixel * CaptureImage.BytesPerPixel + 2];
					const uint8 A = CaptureFrame.Images[Capture_AlphaMask].Pixels[iPixel].A; // Safe to dereference as it's the first image processed
					CaptureImage.Pixels[iPixel] = FColor(R, G, B, A);
				}
			}
			else
			{
				checkNoEntry();
				ErrorCount++;
			}
		}
	});

	MutableThis.bNeedsDataDecompression = false;

	return ErrorCount == 0;
}
