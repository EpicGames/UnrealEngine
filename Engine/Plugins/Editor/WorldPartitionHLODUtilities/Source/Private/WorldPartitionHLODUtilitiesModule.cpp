// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionHLODUtilitiesModule.h"
#include "Modules/ModuleManager.h"

#include "WorldPartition/HLOD/Utilities/WorldPartitionHLODUtilities.h"

IMPLEMENT_MODULE(FWorldPartitionHLODUtilitiesModule, WorldPartitionHLODUtilities);

void FWorldPartitionHLODUtilitiesModule::StartupModule()
{
	Utilities = nullptr;
}

void FWorldPartitionHLODUtilitiesModule::ShutdownModule()
{
	if (Utilities)
	{
		delete Utilities;
		Utilities = nullptr;
	}
}

IWorldPartitionHLODUtilities* FWorldPartitionHLODUtilitiesModule::GetUtilities()
{
	if (Utilities == nullptr)
	{
		Utilities = new FWorldPartitionHLODUtilities();
	}

	return Utilities;
}
