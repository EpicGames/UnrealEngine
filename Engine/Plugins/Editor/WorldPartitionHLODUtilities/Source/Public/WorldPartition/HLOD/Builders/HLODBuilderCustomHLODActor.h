// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "HLODBuilderCustomHLODActor.generated.h"

UCLASS(Blueprintable, Config = Engine, PerObjectConfig)
class UHLODBuilderCustomHLODActorSettings : public UHLODBuilderSettings
{
	GENERATED_UCLASS_BODY()

	virtual bool IsReusingSourceMaterials() const override { return true; }
};

UCLASS(HideDropdown)
class UHLODBuilderCustomHLODActor : public UHLODBuilder
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSubclassOf<UHLODBuilderSettings> GetSettingsClass() const override;
	virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent *>& InSourceComponents) const override;
};