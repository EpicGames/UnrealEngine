// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "HLODBuilderInstancing.generated.h"

#define UE_API WORLDPARTITIONHLODUTILITIES_API

UENUM()
enum class EInstanceFilteringType : uint8
{
	FilterNone UMETA(DisplayName = "No Filtering"),
	FilterMinimumExtent,
	FilterMinimumArea,
	FilterMinimumVolume,
};

UCLASS(MinimalAPI, Blueprintable, Config = Engine, PerObjectConfig)
class UHLODBuilderInstancingSettings : public UHLODBuilderSettings
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const override;

	/**
     * By definition, instancing means reusing the source assets.
     */
	virtual bool IsReusingSourceMaterials() const override { return true; }

	/**
	 * If enabled, the components created for the HLODs will not use Nanite.
	 * Necessary if you want to use the last LOD & the mesh is Nanite enabled, as forced LODs are ignored by Nanite
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Rendering)
	bool bDisallowNanite;

	/**
	 * Type of filtering to apply to instances. Instances that do not pass the filter will be excluded from the HLOD. 
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Filtering)
	EInstanceFilteringType InstanceFilteringType;

	// Minimum extent for an instance to be included in the HLOD. 0 means no minimum.
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Filtering, Meta = (DisplayName = "Minimum Extent", ToolTip = "Minimum Extent in Unreal Units", EditCondition = "InstanceFilteringType==EInstanceFilteringType::FilterMinimumExtent", EditConditionHides))
	double MinimumExtent;

	// Minimum area for an instance to be included in the HLOD. 0 means no minimum. Value is in Unreal Unit ^ 2
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Filtering, Meta = (DisplayName = "Minimum Area (uu²)", ToolTip = "Minimum area in square Unreal Units", EditCondition = "InstanceFilteringType==EInstanceFilteringType::FilterMinimumArea", EditConditionHides))
	double MinimumArea;

	// Minimum volume for an instance to be included in the HLOD. 0 means no minimum. Value is in Unreal Unit ^ 3
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = Filtering, Meta = (DisplayName = "Minimum Volume (uu³)", ToolTip = "Minimum volume in cubic Unreal Units", EditCondition = "InstanceFilteringType==EInstanceFilteringType::FilterMinimumVolume", EditConditionHides))
	double MinimumVolume;
};


/**
 * Build a AWorldPartitionHLOD whose components are ISMC
 */
UCLASS(MinimalAPI, HideDropdown)
class UHLODBuilderInstancing : public UHLODBuilder
{
	 GENERATED_UCLASS_BODY()

public:
	virtual bool RequiresWarmup() const override { return false; }
	virtual bool ShouldIgnoreBatchingPolicy() const override { return true; }
	UE_API virtual TSubclassOf<UHLODBuilderSettings> GetSettingsClass() const override;
	UE_API virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const override;
};

#undef UE_API
