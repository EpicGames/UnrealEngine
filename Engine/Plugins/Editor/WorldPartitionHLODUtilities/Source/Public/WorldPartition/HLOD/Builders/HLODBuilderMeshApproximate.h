// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshMerge/MeshApproximationSettings.h"
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "HLODBuilderMeshApproximate.generated.h"


class UMaterial;
class UMaterialInterface;


UCLASS(MinimalAPI, Blueprintable)
class UHLODBuilderMeshApproximateSettings : public UHLODBuilderSettings
{
	GENERATED_UCLASS_BODY()

	WORLDPARTITIONHLODUTILITIES_API virtual void ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const override;

	/** Mesh approximation settings */
	UPROPERTY(EditAnywhere, Category = HLOD, meta=(EditInline))
	FMeshApproximationSettings MeshApproximationSettings;

	/** Material that will be used by the generated HLOD static mesh */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = HLOD, meta = (DisplayName = "HLOD Material"))
	TObjectPtr<UMaterialInterface> HLODMaterial;
};


/**
 * Build an approximated mesh using geometry from the provided actors
 */
UCLASS(MinimalAPI, HideDropdown)
class UHLODBuilderMeshApproximate : public UHLODBuilder
{
	GENERATED_UCLASS_BODY()

public:
	WORLDPARTITIONHLODUTILITIES_API virtual TSubclassOf<UHLODBuilderSettings> GetSettingsClass() const override;
	WORLDPARTITIONHLODUTILITIES_API virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const override;
};
