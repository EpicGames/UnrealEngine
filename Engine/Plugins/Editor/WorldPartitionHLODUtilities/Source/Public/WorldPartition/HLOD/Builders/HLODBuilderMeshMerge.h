// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshMerge/MeshMergingSettings.h"
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "HLODBuilderMeshMerge.generated.h"


class UMaterial;
class UMaterialInterface;


UCLASS(MinimalAPI, Blueprintable)
class UHLODBuilderMeshMergeSettings : public UHLODBuilderSettings
{
	GENERATED_UCLASS_BODY()

	WORLDPARTITIONHLODUTILITIES_API virtual void ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const override;

	/**
     * Merge mesh will only reuse the source materials when not merging the materials. In this case, the created mesh
     * will have multiple sections, with each of them directly using the source materials.
     */
    virtual bool IsReusingSourceMaterials() const { return !MeshMergeSettings.bMergeMaterials; }

	/** Merged mesh generation settings */
	UPROPERTY(EditAnywhere, Category = HLOD)
	FMeshMergingSettings MeshMergeSettings;

	/** Material that will be used by the generated HLOD static mesh */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = HLOD, meta = (DisplayName = "HLOD Material"))
	TObjectPtr<UMaterialInterface> HLODMaterial;
};


/**
 * Build a merged mesh using geometry from the provided actors
 */
UCLASS(MinimalAPI, HideDropdown)
class UHLODBuilderMeshMerge : public UHLODBuilder
{
	GENERATED_UCLASS_BODY()

public:
	WORLDPARTITIONHLODUTILITIES_API virtual TSubclassOf<UHLODBuilderSettings> GetSettingsClass() const override;
	WORLDPARTITIONHLODUTILITIES_API virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const override;
};
