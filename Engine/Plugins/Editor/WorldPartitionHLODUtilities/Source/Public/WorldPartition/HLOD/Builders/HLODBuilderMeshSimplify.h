// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshMerge/MeshProxySettings.h"
#include "WorldPartition/HLOD/HLODBuilder.h"
#include "HLODBuilderMeshSimplify.generated.h"


class UMaterial;
class UMaterialInterface;


UCLASS(MinimalAPI, Blueprintable)
class UHLODBuilderMeshSimplifySettings : public UHLODBuilderSettings
{
	GENERATED_UCLASS_BODY()

	WORLDPARTITIONHLODUTILITIES_API virtual void ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const override;

	/** Simplified mesh generation settings */
	UPROPERTY(EditAnywhere, Category = HLOD)
	FMeshProxySettings MeshSimplifySettings;

	/** Material that will be used by the generated HLOD static mesh */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = HLOD, meta = (DisplayName = "HLOD Material"))
	TObjectPtr<UMaterialInterface> HLODMaterial;
};


/**
 * Build a simplified mesh using geometry from the provided actors
 */
UCLASS(MinimalAPI, HideDropdown)
class UHLODBuilderMeshSimplify : public UHLODBuilder
{
	GENERATED_UCLASS_BODY()

public:
	WORLDPARTITIONHLODUTILITIES_API virtual TSubclassOf<UHLODBuilderSettings> GetSettingsClass() const override;
	WORLDPARTITIONHLODUTILITIES_API virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const override;
};
