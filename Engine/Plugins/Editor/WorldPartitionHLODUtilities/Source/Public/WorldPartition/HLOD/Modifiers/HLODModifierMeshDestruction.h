// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/HLODModifier.h"
#include "Templates/UniquePtr.h"
#include "HLODModifierMeshDestruction.generated.h"

#define UE_API WORLDPARTITIONHLODUTILITIES_API


class FDestructionMeshMergeExtension;


UCLASS(MinimalAPI)
class UWorldPartitionHLODModifierMeshDestruction : public UWorldPartitionHLODModifier
{
	GENERATED_UCLASS_BODY()

	UE_API virtual bool CanModifyHLOD(TSubclassOf<UHLODBuilder> InHLODBuilderClass) const override;
	UE_API virtual void BeginHLODBuild(const FHLODBuildContext& InHLODBuildContext) override;
	UE_API virtual void EndHLODBuild(TArray<UActorComponent*>&InOutComponents) override;
	UE_API virtual bool MustBeLoadedOnServer() const override;

private:
	// Gathers a list of actors included in the merged mesh
	TUniquePtr<FDestructionMeshMergeExtension> DestructionMeshMergeExtension;

	// Cached from the HLOD Build Context provided to BeginHLODBuild()
	FVector CachedWorldPosition;
};

#undef UE_API
