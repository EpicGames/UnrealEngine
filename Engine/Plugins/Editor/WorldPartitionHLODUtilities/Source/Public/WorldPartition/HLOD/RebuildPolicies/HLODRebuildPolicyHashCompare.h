// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/HLODHashBuilder.h"
#include "WorldPartition/HLOD/HLODRebuildPolicy.h"
#include "HLODRebuildPolicyHashCompare.generated.h"

UCLASS()
class UHLODRebuildPolicyHashData : public UHLODRebuildPolicyData
{
	GENERATED_BODY()

public:
	virtual FString ToString() const override { return HLODHashReport; };

public:
	UPROPERTY()
	uint32 HLODHash;

	// Transient, will be aggregated into the HLOD actor build report
	UPROPERTY(Transient)
	FString HLODHashReport;
};

/**
 * This HLOD rebuild policy computes a hash of various elements in order to detect any change that might
 * impact the resulting HLODs. It is, in effect, extremely sensitive.
 * It will:
 *		APPROVE a rebuild - if the old hash differs from the new hash
 *		REJECT  a rebuild - if the old hash equals the new hash
 */
UCLASS()
class UHLODRebuildPolicyHashCompare : public UHLODRebuildPolicy
{
	GENERATED_BODY()

protected:
	virtual TSubclassOf<UHLODRebuildPolicyData> GetRebuildPolicyDataType() const override { return UHLODRebuildPolicyHashData::StaticClass(); }
	virtual EHLODRebuildPolicyDecision Evaluate(const AWorldPartitionHLOD* InHLODActor, const UHLODRebuildPolicyData* InOldData, const UHLODRebuildPolicyData* InNewData, FString& OutReason) const override;
	virtual UHLODRebuildPolicyData* ComputeDataForRebuildPolicy(const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents, const UHLODRebuildPolicyData* InExistingPolicyData) const override;

private:
	void ComputeHLODHash(FHLODHashBuilder& InHashBuilder, const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents) const;
};
