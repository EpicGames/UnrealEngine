// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/HLODRebuildPolicy.h"
#include "HLODRebuildPolicyImageCompare.generated.h"


UENUM()
enum EHLODCaptureImageType : uint8
{
	Capture_AlphaMask,	// 8-bits
	Capture_BaseColor,	// 24-bits
	Capture_Normal,		// 24-bits
	Capture_Emissive,	// 24-bits
	Capture_Metallic,	// 8-bits
	Capture_Roughness,	// 8-bits
	Capture_Specular,	// 8-bits
	Num
};

template<EHLODCaptureImageType T> struct THLODCaptureImageTraits { static constexpr int32 NumChannels = 1; };
template<> struct THLODCaptureImageTraits<Capture_BaseColor>	 { static constexpr int32 NumChannels = 3; };
template<> struct THLODCaptureImageTraits<Capture_Normal>		 { static constexpr int32 NumChannels = 3; };
template<> struct THLODCaptureImageTraits<Capture_Emissive>		 { static constexpr int32 NumChannels = 3; };

USTRUCT()
struct FHLODCaptureImage
{
	GENERATED_BODY()

	FHLODCaptureImage()
	: Width(0)
	, Height(0)
	, BytesPerPixel(0)
	{}

	UPROPERTY()
	int32 Width;

	UPROPERTY()
	int32 Height;

	UPROPERTY()
	uint8 BytesPerPixel;

	UPROPERTY()
	TArray<uint8> PixelsCompressed;	// Compressed, minimal number of channels (BytesPerPixel), saved to disk

	UPROPERTY(Transient)
	TArray<FColor> Pixels;			// Uncompressed, RGBA, not saved to disk
};

USTRUCT()
struct FHLODCaptureFrame
{
	GENERATED_BODY()

	FHLODCaptureFrame()
		: CameraLocation(ForceInitToZero)
		, CameraRotation(ForceInitToZero)
		, OrthoWidth(0.0f)
	{
	}

	UPROPERTY()
	FVector CameraLocation;

	UPROPERTY()
	FRotator CameraRotation;

	UPROPERTY()
	float OrthoWidth;

	UPROPERTY()
	FHLODCaptureImage Images[EHLODCaptureImageType::Num];
};

UCLASS()
class UHLODRebuildPolicyImageData : public UHLODRebuildPolicyData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FBox CaptureBounds;

	UPROPERTY()
	TArray<FHLODCaptureFrame> CaptureFrames;

	UPROPERTY()
	uint32 HLODLayerHash;

	UPROPERTY(Transient)
	bool bNeedsDataCompression = false;

	UPROPERTY(Transient)
	bool bNeedsDataDecompression = true;

	void CompressData() const;
	bool DecompressData() const;

protected:
	//~ Begin UObject Interface.
	virtual void PreSave(FObjectPreSaveContext InObjectSaveContext) override;
	//~ End UObject Interface.
};

/**
 * This HLOD rebuild policy takes screen captures of the SOURCE actors over multiple angles (and for different GBuffer properties)
 * It will then perform an SSIM evaluation in order to assess if there is a significant visual change between the old and new data set.
 * It will:
 *		APPROVE a rebuild - if the difference between old vs new images is greater than the configured threshold
 *		REJECT  a rebuild - if the difference between old vs new images is less than the configured threshold
 */
UCLASS(MinimalAPI, Config = Editor)
class UHLODRebuildPolicyImageCompare : public UHLODRebuildPolicy
{
	GENERATED_UCLASS_BODY()

protected:
	virtual TSubclassOf<UHLODRebuildPolicyData> GetRebuildPolicyDataType() const override { return UHLODRebuildPolicyImageData::StaticClass(); }
	virtual EHLODRebuildPolicyDecision Evaluate(const AWorldPartitionHLOD* InHLODActor, const UHLODRebuildPolicyData* InOldData, const UHLODRebuildPolicyData* InNewData, FString& OutReason) const override;
	virtual UHLODRebuildPolicyData* ComputeDataForRebuildPolicy(const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents, const UHLODRebuildPolicyData* InExistingPolicyData) const override;

	UPROPERTY(Config)
	uint32 MaxImageSize = 512;

	UPROPERTY(Config)
	uint32 WarmupFrameCount = 5;

	UPROPERTY(Config)
	float CaptureImageMargin = 0.10f;

	UPROPERTY(Config)
	float MaximumLocalError = 1.00f;

	UPROPERTY(Config)
	float MaximumGlobalError = 0.015f;
};
