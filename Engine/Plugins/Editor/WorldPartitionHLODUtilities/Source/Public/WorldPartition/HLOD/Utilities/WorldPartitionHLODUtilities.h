// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/IWorldPartitionHLODUtilities.h"

/**
* FWorldPartitionHLODUtilities implementation
*/
class FWorldPartitionHLODUtilities : public IWorldPartitionHLODUtilities
{
public:
	virtual TArray<AWorldPartitionHLOD*> CreateHLODActors(FHLODCreationContext& InCreationContext, const FHLODCreationParams& InCreationParams, const TArray<IStreamingGenerationContext::FActorInstance>& InActors) override;
	virtual bool BuildHLOD(const FHLODBuildParams& InBuildParams) override;
	virtual TSubclassOf<UHLODBuilder> GetHLODBuilderClass(const UHLODLayer* InHLODLayer) override;
	virtual UHLODBuilderSettings* CreateHLODBuilderSettings(UHLODLayer* InHLODLayer) override;
	virtual void SetHLODBuildEvaluator(IWorldPartitionHLODUtilities::FHLODBuildEvaluator BuildEvaluatorDelegate) override;

protected:
	IWorldPartitionHLODUtilities::FHLODBuildEvaluator HLODBuildEvaluatorDelegate;
};
