// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/IWorldPartitionHLODUtilitiesModule.h"

#define UE_API WORLDPARTITIONHLODUTILITIES_API

/**
* IWorldPartitionHLODUtilities module interface
*/
class FWorldPartitionHLODUtilitiesModule : public IWorldPartitionHLODUtilitiesModule
{
public:
	UE_API virtual void ShutdownModule() override;
	UE_API virtual void StartupModule() override;

	UE_API virtual IWorldPartitionHLODUtilities* GetUtilities() override;

private:
	IWorldPartitionHLODUtilities* Utilities;
};

#undef UE_API
