// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System.IO;

public class WorldPartitionHLODUtilities : ModuleRules
{
    public WorldPartitionHLODUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		// Some files were initially right under /Public and were moved to sub directories
		// Ensure we don't break old-style includes
		PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Public/WorldPartition/HLOD/Builders"));
		PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Public/WorldPartition/HLOD/Utilities"));
		
		PublicDependencyModuleNames.AddRange(
            new string[]
			{
				"Core",
				"CoreUObject"
			}
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
			{
				"Engine",
				"ImageWrapper",
				"ImageWriteQueue",
				"Json",
				"JsonUtilities",
				"MaterialBaking",
				"MaterialUtilities",
				"MeshDescription",
				"MeshMergeUtilities",
				"RenderCore",
				"RHI",
				"ScreenShotComparison",
				"ScreenShotComparisonTools",
				"Slate", 
				"SlateCore",
				"StaticMeshDescription",
				"WorldPartitionEditor"
			}
        );

        PrivateIncludePathModuleNames.AddRange(
            new string[]
            {
				"GeometryProcessingInterfaces"
			}
        );

        DynamicallyLoadedModuleNames.AddRange(
            new string[]
            {
                "MeshUtilities",
                "MeshReductionInterface",
				"GeometryProcessingInterfaces",
			}
        );
	}
}
