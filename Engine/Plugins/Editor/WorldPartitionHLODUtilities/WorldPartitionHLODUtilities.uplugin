{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "World Partition HLOD Utilities",
	"Description" : "Editor utility classes & HLOD layer asset types",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "WorldPartitionHLODUtilities",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}